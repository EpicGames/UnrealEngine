// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISubsonicEventRegistry.h"

namespace UE::Subsonic::Core
{
	TUniquePtr<ISubsonicEventRegistry> ISubsonicEventRegistry::Instance;

	ISubsonicEventRegistry* ISubsonicEventRegistry::Get()
	{
		return Instance.Get();
	}

	ISubsonicEventRegistry& ISubsonicEventRegistry::GetChecked()
	{
		checkf(Instance.IsValid(), TEXT("No subsystem event registry initialized (invalid access)"));
		return *Instance.Get();
	}

	void ISubsonicEventRegistry::Initialize(TUniquePtr<ISubsonicEventRegistry>&& InRegistry)
	{
		checkf(!Instance.IsValid(), TEXT("Cannot reinitialize Subsonic Event Registry (Initialize has been called more than once prior to deinitialization"));
		Instance = MoveTemp(InRegistry);
	}

	void ISubsonicEventRegistry::Deinitialize()
	{
		checkf(Instance.IsValid(), TEXT("Cannot Deinitialize Subsonic Event Registry (Never initialized or Deinitialize has been called more than once"));
		Instance.Reset();
	}
} // namespace UE::Subsonic::Core
