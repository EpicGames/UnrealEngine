// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "SubsonicBindingUtils.h"

#include "Algo/AllOf.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "StructUtils/PropertyBag.h"
#include "SubsonicEventCollection.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "SubsonicBindingUtils"


namespace UE::Subsonic::Core::BindingUtils
{
	bool ArePropertiesBindingCompatible(const FProperty* FromBag, const FProperty* ToAction)
	{
		check(FromBag);
		check(ToAction);

		if (ToAction->SameType(FromBag))
		{
			return true;
		}
		const FObjectPropertyBase* ActionObj = CastField<FObjectPropertyBase>(ToAction);
		const FObjectPropertyBase* BagObj = CastField<FObjectPropertyBase>(FromBag);

		return ActionObj && BagObj && BagObj->PropertyClass->IsChildOf(ActionObj->PropertyClass);
	}

#if WITH_EDITORONLY_DATA
	FStaleBindingsMap FindStaleBindings(
		const FSubsonicEventCollectionDefinition& Definition, const FCollectionHandle& ParentHandle)
	{
		FStaleBindingsMap StaleBindings;

		for (const TPair<FGameplayTag, FSubsonicEvent>& EventPair : Definition.GetEvents())
		{
			const FSubsonicEvent& Event = EventPair.Value;
			const FEventHandle EventHandle = { .Collection = ParentHandle, .EventName = EventPair.Key.GetTagName() };

			int32 ActionIndex = 0;
			for (const FSubsonicEventActionDefinition& ActionDef : Event.GetActionCollection())
			{
				const FActionHandle ActionHandle = { .Event = EventHandle, .Index = ActionIndex++ };
				const TInstancedStruct<FSubsonicEventActionBase>& Action = ActionDef.GetAction();
				const FSubsonicEventActionBase* ActionBase = Action.GetPtr();
				const UScriptStruct* ActionStructType = Action.GetScriptStruct();
				if (!ActionBase || !ActionStructType)
				{
					continue;
				}

				for (const TPair<FName, FSubsonicEventActionBoundProperties>& BindingPair : ActionBase->GetBindings())
				{
					const FName& ParamName = BindingPair.Key;
					const FPropertyBagPropertyDesc* BagDesc = nullptr;
					if (const UPropertyBag* EventBag = Event.GetParameters().GetPropertyBagStruct())
					{
						BagDesc = EventBag->FindPropertyDescByName(ParamName);
					}
					if (!BagDesc)
					{
						if (const UPropertyBag* CollBag = Definition.GetParameters().GetPropertyBagStruct())
						{
							BagDesc = CollBag->FindPropertyDescByName(ParamName);
						}
					}

					const bool bNoParamFound = !BagDesc || !BagDesc->CachedProperty;
					for (const FName& BoundPropName : BindingPair.Value.Properties)
					{
						if (!bNoParamFound)
						{
							const FProperty* ActionProp = ActionStructType->FindPropertyByName(BoundPropName);
							if (ActionProp && BagDesc->CachedProperty && ArePropertiesBindingCompatible(BagDesc->CachedProperty, ActionProp))
							{
								continue;
							}
						}

						StaleBindings.FindOrAdd(ParamName).Add(FStalePropertyHandle { .Action = ActionHandle, .Property = BoundPropName });
					}
				}
			}
		}

		return StaleBindings;
	}

	FName FindRenameCandidate(const FSubsonicEventCollectionDefinition& Definition, FName OldParamName, const TArray<FStalePropertyHandle>& StaleEntries)
	{
		// Collect the set of action property types that were bound to the old parameter name.
		TArray<const FProperty*> StaleActionProps;
		for (const FStalePropertyHandle& Entry : StaleEntries)
		{
			if (const FSubsonicEventActionDefinition* ActionDef = Definition.FindAction(Entry.Action))
			{
				if (const UScriptStruct* ActionStruct = ActionDef->GetAction().GetScriptStruct())
				{
					if (const FProperty* ActionProp = ActionStruct->FindPropertyByName(Entry.Property))
					{
						StaleActionProps.AddUnique(ActionProp);
					}
				}
			}
		}

		if (StaleActionProps.IsEmpty())
		{
			return NAME_None;
		}

		// Collect all parameter names referenced by ANY binding across the ENTIRE collection,
		// so that pre-existing parameters bound elsewhere are excluded as rename candidates.
		TSet<FName> ExistingBoundParams;
		for (const TPair<FGameplayTag, FSubsonicEvent>& EventPair : Definition.GetEvents())
		{
			for (const FSubsonicEventActionDefinition& ActionDef : EventPair.Value.GetActionCollection())
			{
				if (const FSubsonicEventActionBase* ActionBase = ActionDef.GetAction().GetPtr())
				{
					for (const TPair<FName, FSubsonicEventActionBoundProperties>& BindingPair : ActionBase->GetBindings())
					{
						ExistingBoundParams.Add(BindingPair.Key);
					}
				}
			}
		}

		// Scan a single bag for exactly one candidate parameter that:
		// 1. Is NOT the old name
		// 2. Is NOT already referenced by an existing binding
		// 3. Is type-compatible with the stale action properties
		// Returns NAME_None if zero or multiple candidates (ambiguous).
		auto FindCandidateInBag = [&](const UPropertyBag* Bag)
		{
			const FName* BagCandidate = nullptr;
				
			if (Bag)
			{
				for (const FPropertyBagPropertyDesc& Desc : Bag->GetPropertyDescs())
				{
					if (Desc.Name != OldParamName && !ExistingBoundParams.Contains(Desc.Name) && Desc.CachedProperty)
					{
						const bool bAllCompatible = Algo::AllOf(StaleActionProps, [&Desc](const FProperty* ActionProp)
						{
							return ArePropertiesBindingCompatible(Desc.CachedProperty, ActionProp);
						});

						if (bAllCompatible)
						{
							if (!BagCandidate || *BagCandidate == Desc.Name)
							{
								BagCandidate = &Desc.Name;
							}
							else
							{
								// Ambiguous within this bag
								BagCandidate = nullptr;
								break;
							}
						}
					}
				}
			}
			return BagCandidate;
		};

		// Check collection bag first (most common rename target), then each event bag independently.
		// Scanning per-bag prevents unrelated parameters in other bags from creating false ambiguity.
		if (const FName* Result = FindCandidateInBag(Definition.GetParameters().GetPropertyBagStruct()))
		{
			return *Result;
		}

		for (const TPair<FGameplayTag, FSubsonicEvent>& EventPair : Definition.GetEvents())
		{
			if (const FName* Result = FindCandidateInBag(EventPair.Value.GetParameters().GetPropertyBagStruct()))
			{
				return *Result;
			}
		}

		return NAME_None;
	}
#endif // WITH_EDITORONLY_DATA

	void PromptForStaleBindings(const FSubsonicEventCollectionDefinition& Definition, const FStaleBindingsMap& StaleBindings, EStaleBindingResponse& OutResponse)
	{
#if WITH_EDITORONLY_DATA
		// Detect rename candidates for each stale parameter.
		TMap<FName, FName> RenameMap;
		for (const TPair<FName, TArray<FStalePropertyHandle>>& Pair : StaleBindings)
		{
			const FName Candidate = FindRenameCandidate(Definition, Pair.Key, Pair.Value);
			if (!Candidate.IsNone())
			{
				RenameMap.Add(Pair.Key, Candidate);
			}
		}

		const bool bCanRebind = RenameMap.Num() == StaleBindings.Num();

		if (bCanRebind)
		{
			// All stale params have rename candidates — offer the rebind option.
			TArray<FText> RenameTexts;
			for (const TPair<FName, FName>& Pair : RenameMap)
			{
				// \u2192 is the arrow character
				RenameTexts.Add(FText::Format(LOCTEXT("RenameMapping", "{0} \u2192 {1}"),
					FText::FromName(Pair.Key), FText::FromName(Pair.Value)));
			}
			const FText RenameList = FText::Join(FText::FromString(TEXT("\r\n\t* ")), RenameTexts);

			const FText Title = LOCTEXT("StaleBindingRename_Title", "Parameter Renamed");
			const FText Message = FText::Format(
				LOCTEXT("StaleBindingRenameWarning",
					"The following parameter(s) have been renamed:\r\n\t{0}\r\n"
					"- Update affected action property bindings to the new name? (Yes)\r\n"
					"- Remove the stale bindings? (No)\r\n"
					"- Cancel the parameter rename? (Cancel)"),
				RenameList);

			const EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNoCancel, Message, Title);
			switch (Result)
			{
				case EAppReturnType::Yes:
				{
					OutResponse = EStaleBindingResponse::Rebind;
				}
				break;

				case EAppReturnType::No:
				{
					OutResponse = EStaleBindingResponse::RemoveBindings;
				}
				break;

				case EAppReturnType::Cancel:
				default:
				{
					OutResponse = EStaleBindingResponse::Revert;
				}
				break;
			}
		}
		else
		{
			// No rename candidates — show the standard removal/revert dialog.
			TArray<FText> ParamTexts;
			Algo::Transform(StaleBindings, ParamTexts, [](const TPair<FName, TArray<FStalePropertyHandle>>& Pair) { return FText::FromName(Pair.Key); });
			const FText ParamList = FText::Join(FText::FromString(TEXT("\r\n\t* ")), ParamTexts);

			const FText Title = LOCTEXT("StaleBindingWarning_Title", "Binding Removal Required");
			const FText Message = FText::Format(
				LOCTEXT("StaleBindingWarning",
					"The following parameter(s) have property bindings that are no longer valid "
					"due to removal or type/name change:\r\n\t{0}\r\n\r\n"
					"Would you like to remove the affected bindings (Yes) or revert the parameter change (No)?"),
				ParamList);

			const EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, Message, Title);
			OutResponse = Result == EAppReturnType::No
				? EStaleBindingResponse::Revert
				: EStaleBindingResponse::RemoveBindings;
		}
#endif // WITH_EDITORONLY_DATA
	}
} // namespace UE::Subsonic::Core::BindingUtils

#undef LOCTEXT_NAMESPACE
#endif // WITH_EDITOR
