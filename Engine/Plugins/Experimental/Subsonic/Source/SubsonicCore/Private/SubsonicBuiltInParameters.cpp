// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicBuiltInParameters.h"

namespace UE::Subsonic::BuiltInParameters
{
	const FName Volume         = "Volume";
	const FName PitchShift     = "PitchShift";
	const FName HighpassCutoff = "HighpassCutoff";
	const FName LowpassCutoff  = "LowpassCutoff";
	const FName FadeOutTime    = "FadeOutTime";

} // namespace UE::Subsonic::BuiltInParameters
