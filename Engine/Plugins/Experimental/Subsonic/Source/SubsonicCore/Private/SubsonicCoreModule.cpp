// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "SubsonicCoreLog.h"

DEFINE_LOG_CATEGORY(LogSubsonic);

IMPLEMENT_MODULE(FDefaultModuleImpl, SubsonicCore)
