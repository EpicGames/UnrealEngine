// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicEventCollection.h"

#include "ISubsonicEventRegistry.h"
#include "NativeGameplayTags.h"
#include "StructUtils/StructView.h"
#include "SubsonicBindingUtils.h"
#include "SubsonicCoreLog.h"
#include "SubsonicExecutor.h"
#include "SubsonicHandles.h"
#include "UObject/UnrealType.h"


namespace UE::Subsonic::Core
{
	UE_DEFINE_GAMEPLAY_TAG(TAG_SubsonicCore, "Subsonic");
	UE_DEFINE_GAMEPLAY_TAG(TAG_SubsonicCore_Event_Play, "Subsonic.Event.Play");
	UE_DEFINE_GAMEPLAY_TAG(TAG_SubsonicCore_Event_Stop, "Subsonic.Event.Stop");

	namespace CollectionPrivate
	{
		constexpr bool bErrorOnMissingTag =
#if WITH_EDITOR
		// Validation will catch missing tags. While fixing up tags or
		// creating new events with an unset tag, this messaging is too pervasive.
		false;
#else // !WITH_EDITOR
		true;
#endif // !WITH_EDITOR
		constexpr uint32 InvalidId = static_cast<uint32>(INDEX_NONE);
	} // namespace CollectionPrivate

#if WITH_EDITOR
	const TMap<FName, FSubsonicEventActionBoundProperties>& FSubsonicEventActionBase::GetBindings() const
	{
		return Bindings;
	}

	FName FSubsonicEventActionBase::GetBindingsParameterName()
	{
		return GET_MEMBER_NAME_CHECKED(FSubsonicEventActionBase, Bindings);
	}

	FName FSubsonicEventActionBase::GenerateDefaultBaseIdentifier(const FEventHandle& OwnerHandle)
	{
		const bool bIsPlayEvent = OwnerHandle.EventName == TAG_SubsonicCore_Event_Play.GetTag().GetTagName();
		const bool bIsStopEvent = OwnerHandle.EventName == TAG_SubsonicCore_Event_Stop.GetTag().GetTagName();
		FName DefaultName = bIsPlayEvent || bIsStopEvent || OwnerHandle.EventName.IsNone()
			? OwnerHandle.Collection.CollectionName
			: OwnerHandle.EventName;
		TArray<FString> Tokens;
		DefaultName.ToString().ParseIntoArray(Tokens, TEXT("/"));
		if (!Tokens.IsEmpty())
		{
			const FString Last = MoveTemp(Tokens.Last());
			Last.ParseIntoArray(Tokens, TEXT("."));
			DefaultName = Tokens.IsEmpty()
				? *Last
				: *Tokens.Last();
		}
		return DefaultName;
	}

	void FSubsonicEventActionBase::InitializeDefaultActionName(TInstancedStruct<FSubsonicEventActionBase>& ActionConfig, const FEventHandle& OwnerHandle)
	{
		if (FSubsonicEventActionBase* Action = ActionConfig.GetMutablePtr())
		{
			const FName BaseName = GenerateDefaultBaseIdentifier(OwnerHandle);
			Action->InitializeDefaultName(ActionConfig.GetScriptStruct(), BaseName);
		}
	}

	void FSubsonicEventActionBase::InitializeDefaultName(const UScriptStruct* ActionStruct, FName BaseName)
	{
		if (!ActionStruct)
		{
			return;
		}

		static const FLazyName Name("Name");
		for (TFieldIterator<FProperty> PropIt(ActionStruct); PropIt; ++PropIt)
		{
			if (PropIt->GetOwnerStruct() != ActionStruct)
			{
				break;
			}

			if (PropIt->GetFName() == Name)
			{
				if (FNameProperty* NameProp = CastField<FNameProperty>(*PropIt))
				{
					NameProp->SetPropertyValue_InContainer(this, BaseName);
				}
				break;
			}
		}
	}

	FText FSubsonicEventActionBase::GetDisplayInfo() const
	{
		return FText();
	}

	FName FSubsonicEventActionBase::GetBoundParameterForProperty(FName PropertyName) const
	{
#if WITH_EDITORONLY_DATA
		for (const TPair<FName, FSubsonicEventActionBoundProperties>& Pair : Bindings)
		{
			if (Pair.Value.Properties.Contains(PropertyName))
			{
				return Pair.Key;
			}
		}
#endif // WITH_EDITORONLY_DATA
		return NAME_None;
	}

	void FSubsonicEventActionBase::AddPropertyBinding(FName ParameterName, FName PropertyName)
	{
#if WITH_EDITORONLY_DATA
		RemovePropertyBinding(PropertyName);
		Bindings.FindOrAdd(ParameterName).Properties.AddUnique(PropertyName);
#endif // WITH_EDITORONLY_DATA
	}

	bool FSubsonicEventActionBase::RemoveAllPropertyBindingsToParameter(FName ParameterName)
	{
		return Bindings.Remove(ParameterName) > 0;
	}

	void FSubsonicEventActionBase::RemovePropertyBinding(FName PropertyName)
	{
#if WITH_EDITORONLY_DATA
		TArray<FName> ToRemove;
		for (TPair<FName, FSubsonicEventActionBoundProperties>& Pair : Bindings)
		{
			Pair.Value.Properties.Remove(PropertyName);
			if (Pair.Value.Properties.IsEmpty())
			{
				ToRemove.Add(Pair.Key);
			}
		}

		for (FName Name : ToRemove)
		{
			Bindings.Remove(Name);
		}
#endif // WITH_EDITORONLY_DATA
	}
#endif // WITH_EDITOR

	const TInstancedStruct<FSubsonicEventActionBase>& FSubsonicEventActionDefinition::GetAction() const
	{
		return Action;
	}

#if WITH_EDITOR
	void FSubsonicEventActionDefinition::SetAction(TInstancedStruct<FSubsonicEventActionBase> NewAction)
	{
		Action = MoveTemp(NewAction);
	}

	FSubsonicEventActionBase* FSubsonicEventActionDefinition::GetMutableActionBase()
	{
		return Action.GetMutablePtr();
	}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	const FInstancedPropertyBag& FSubsonicEvent::GetParameters() const
	{
		return Parameters;
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	void FSubsonicEvent::BindParameter(const FInstancedPropertyBag& InSourceParameters, FName ParamName)
	{
		const FInstancedPropertyBag* EffectiveBag = nullptr;
		auto GetPropertyBagDesc = [&ParamName](const FInstancedPropertyBag& Bag)
		{
			const FPropertyBagPropertyDesc* Desc = nullptr;
			if (const UPropertyBag* BagStruct = Bag.GetPropertyBagStruct())
			{
				Desc = BagStruct->FindPropertyDescByName(ParamName);
			}

			return Desc;
		};

		const FPropertyBagPropertyDesc* PropertyDesc = GetPropertyBagDesc(Parameters);
		if (PropertyDesc)
		{
			EffectiveBag = &Parameters;
		}
		else
		{
			PropertyDesc = GetPropertyBagDesc(InSourceParameters);
			EffectiveBag = &InSourceParameters;
		}

		if (!PropertyDesc || !PropertyDesc->CachedProperty)
		{
			return;
		}

		check(EffectiveBag);
		const FConstStructView BagView = EffectiveBag->GetValue();

		for (FSubsonicEventActionDefinition& ActionDefinition : ActionCollection)
		{
			const UScriptStruct* ActionStructType = ActionDefinition.Action.GetScriptStruct();
			FSubsonicEventActionBase* ActionPtr = ActionDefinition.Action.GetMutablePtr();
			if (!ActionPtr || !ActionStructType)
			{
				continue;
			}

			// Propagate parameter value to all type-compatible action properties.
			// This does not record entries in the Bindings map — only explicit bindings
			// created via AddPropertyBinding (the editor UI) are tracked there.
			// Skip properties that are explicitly bound to a different parameter so that
			// auto-propagation of one parameter cannot overwrite an explicit binding's value.
			for (TFieldIterator<FProperty> PropIt(ActionStructType); PropIt; ++PropIt)
			{
				FProperty* ActionProp = *PropIt;
				if (ActionProp->HasMetaData("NoBinding"))
				{
					continue;
				}

				// Require compatible property types (including inner types for containers) to safely copy the value.
				if (!BindingUtils::ArePropertiesBindingCompatible(PropertyDesc->CachedProperty, ActionProp))
				{
					continue;
				}

				// Only propagate to properties explicitly bound to this parameter.
				const FName BoundParam = ActionPtr->GetBoundParameterForProperty(ActionProp->GetFName());
				if (BoundParam != ParamName)
				{
					continue;
				}

				if (const void* SrcPtr = PropertyDesc->CachedProperty->ContainerPtrToValuePtr<const void>(BagView.GetMemory()); ensure(SrcPtr))
				{
					void* DstPtr = ActionProp->ContainerPtrToValuePtr<void>(ActionPtr);
					ActionProp->CopyCompleteValue(DstPtr, SrcPtr);
				}
			}
		}
	}

	void FSubsonicEvent::UnbindParameter(FName ParamName)
	{
		for (FSubsonicEventActionDefinition& ActionDefinition : ActionCollection)
		{
			if (FSubsonicEventActionBase* ActionPtr = ActionDefinition.Action.GetMutablePtr())
			{
				ActionPtr->RemoveAllPropertyBindingsToParameter(ParamName);
			}
		}
	}
#endif // WITH_EDITOR

	void FSubsonicEvent::Execute(const FSubsonicExecutor& InExecutor, const FEventHandle& InHandle) const
	{
		checkf(InExecutor.GetCollection() && InExecutor.GetCollection()->GetCollectionId() == InHandle.Collection.CollectionId,
			TEXT("Invalid execution context: executor Collection must be valid and Id must match that of the given event handle"));

		FActionHandle ActionHandle { .Event = InHandle };
		for (int32 Index = 0; Index < ActionCollection.Num(); ++Index)
		{
			ActionHandle.Index = Index;

			TConstStructView<FSubsonicEventActionBase> Action = ActionCollection[Index].Action;
			if (const FSubsonicEventActionBase* ActionPtr = Action.GetPtr())
			{
				ActionPtr->Execute(InExecutor, ActionHandle);
			}
			else
			{
				UE_LOGF(LogSubsonic, Warning, "Action not specified: Skipping action at '%ls'", *ActionHandle.ToString());
			}
		}
	}

	const TArray<FSubsonicEventActionDefinition>& FSubsonicEvent::GetActionCollection() const
	{
		return ActionCollection;
	}

#if WITH_EDITOR
	FName FSubsonicEvent::GetActionCollectionPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FSubsonicEvent, ActionCollection);
	}

	FName FSubsonicEvent::GetParametersPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FSubsonicEvent, Parameters);
	}
#endif // WITH_EDITOR

#if WITH_EDITOR
	bool FSubsonicEvent::GetAutoAudition() const
	{
		return bAutoAudition;
	}
#endif // WITH_EDITOR

	bool FSubsonicEvent::GetIsPublic() const
	{
		return bIsPublic;
	}

#if WITH_EDITOR
	TArray<FSubsonicEventActionDefinition>& FSubsonicEvent::GetMutableActionCollection()
	{
		return ActionCollection;
	}

	void FSubsonicEvent::SetAutoAudition(bool bInAutoAudition)
	{
		bAutoAudition = bInAutoAudition;
	}

	void FSubsonicEvent::SetIsPublic(bool bInIsPublic)
	{
		bIsPublic = bInIsPublic;
	}
#endif // WITH_EDITOR

	FSubsonicEventCollectionDefinition::FSubsonicEventCollectionDefinition(FSubsonicEventCollectionDefinition&& InOther)
	{
		if (this != &InOther)
		{
			Unregister();

			CollectionId = InOther.CollectionId;
			InOther.CollectionId = GetInvalidId();

			DeviceId = InOther.DeviceId;
			InOther.DeviceId = INDEX_NONE;

			Events = MoveTemp(InOther.Events);
			InOther.Events.Reset();

#if WITH_EDITORONLY_DATA
			Parameters = MoveTemp(InOther.Parameters);
			InOther.Parameters.Reset();
#endif // WITH_EDITORONLY_DATA
		}
	}

	FSubsonicEventCollectionDefinition& FSubsonicEventCollectionDefinition::operator=(FSubsonicEventCollectionDefinition&& InOther)
	{
		if (&InOther != this)
		{
			Unregister();

			CollectionId = InOther.CollectionId;
			InOther.CollectionId = GetInvalidId();

			DeviceId = InOther.DeviceId;
			InOther.DeviceId = INDEX_NONE;

			Events = MoveTemp(InOther.Events);
			InOther.Events.Reset();

#if WITH_EDITORONLY_DATA
			Parameters = MoveTemp(InOther.Parameters);
			InOther.Parameters.Reset();
#endif // WITH_EDITORONLY_DATA
		}

		return *this;
	}

	void FSubsonicEventCollectionDefinition::AssignId()
	{
		static uint32 MaxId = CollectionPrivate::InvalidId;
		CollectionId = ++MaxId;
	}

#if WITH_EDITOR
	FSubsonicEventActionDefinition* FSubsonicEventCollectionDefinition::AddAction(const FEventHandle& InHandle, int32 InsertIndex)
	{
		if (FSubsonicEvent* Event = FindMutableEvent(InHandle))
		{
			TArray<FSubsonicEventActionDefinition>& ActionCollection = Event->GetMutableActionCollection();
			if (!ActionCollection.IsValidIndex(InsertIndex))
			{
				return &ActionCollection.AddDefaulted_GetRef();
			}

			return &ActionCollection.InsertDefaulted_GetRef(InsertIndex);
		}

		return nullptr;
	}

	FSubsonicEvent* FSubsonicEventCollectionDefinition::AddEvent(const FGameplayTag& InTag, FSubsonicEvent NewEvent)
	{
		if (ContainsEvent(InTag))
		{
			return nullptr;
		}

		return &Events.Add(InTag, MoveTemp(NewEvent));
	}

	void FSubsonicEventCollectionDefinition::BindAllParameters()
	{
		if (const UPropertyBag* BagStruct = Parameters.GetPropertyBagStruct())
		{
			for (const FPropertyBagPropertyDesc& Desc : BagStruct->GetPropertyDescs())
			{
				BindParameter(Desc.Name);
			}
		}

		// Bind event-level parameters that have no collection-level counterpart
		for (TPair<FGameplayTag, FSubsonicEvent>& Pair : Events)
		{
			const UPropertyBag* EventBagStruct = Pair.Value.Parameters.GetPropertyBagStruct();
			if (!EventBagStruct)
			{
				continue;
			}
			for (const FPropertyBagPropertyDesc& Desc : EventBagStruct->GetPropertyDescs())
			{
				const UPropertyBag* CollectionBagStruct = Parameters.GetPropertyBagStruct();
				if (!CollectionBagStruct || !CollectionBagStruct->FindPropertyDescByName(Desc.Name))
				{
					Pair.Value.BindParameter(Parameters, Desc.Name);
				}
			}
		}
	}

	void FSubsonicEventCollectionDefinition::BindParameter(FName ParamName)
	{
		for (TPair<FGameplayTag, FSubsonicEvent>& Pair : Events)
		{
			Pair.Value.BindParameter(Parameters, ParamName);
		}
	}

	bool FSubsonicEventCollectionDefinition::RemoveAllPropertyBindingsToParameter(FName ParamName)
	{
		bool bBindingsRemoved = false;

#if WITH_EDITORONLY_DATA
		for (TPair<FGameplayTag, FSubsonicEvent>& EventPair : Events)
		{
			for (FSubsonicEventActionDefinition& ActionDef : EventPair.Value.GetMutableActionCollection())
			{
				if (FSubsonicEventActionBase* ActionBase = ActionDef.GetMutableActionBase())
				{
					bBindingsRemoved |= ActionBase->RemoveAllPropertyBindingsToParameter(ParamName);
				}
			}
		}
#endif // WITH_EDITORONLY_DATA

		return bBindingsRemoved;
	}

	bool FSubsonicEventCollectionDefinition::RemoveStaleBindings(const FCollectionHandle& ParentHandle, bool bPromptRemoval)
	{
#if WITH_EDITORONLY_DATA
		using EStaleBindingResponse = BindingUtils::EStaleBindingResponse;
		using FStalePropertyHandle = BindingUtils::FStalePropertyHandle;

		// Check for stale bindings caused by structural changes (type or name changes) to parameter bags.
		const TMap<FName, TArray<FStalePropertyHandle>> StaleBindings = BindingUtils::FindStaleBindings(*this, ParentHandle);
		if (!StaleBindings.IsEmpty())
		{
			if (bPromptRemoval)
			{ 
				EStaleBindingResponse Response = EStaleBindingResponse::RemoveBindings;
				OnStaleBindingsDetected.Broadcast(StaleBindings, Response);
				if (Response == EStaleBindingResponse::Revert)
				{
					// If an editor is open, it is expected to respond to the delegate above and undo the associated transaction in the
					// next tick. A response of revert essentially signals the remaining removal to be skipped. (If no editor is open,
					// stale bindings are removed and stale bindings are removed on next editor load).
					return false;
				}

				if (Response == EStaleBindingResponse::Rebind)
				{
					// Update binding keys from old parameter name to the detected rename target.
					// Falls back to removal for any parameter where no unique candidate is found.
					for (const TPair<FName, TArray<FStalePropertyHandle>>& Pair : StaleBindings)
					{
						const FName NewParamName = BindingUtils::FindRenameCandidate(*this, Pair.Key, Pair.Value);
						for (const FStalePropertyHandle& Entry : Pair.Value)
						{
							if (FSubsonicEventActionDefinition* Action = FindMutableAction(Entry.Action))
							{
								if (FSubsonicEventActionBase* ActionBase = Action->GetMutableActionBase())
								{
									ActionBase->RemovePropertyBinding(Entry.Property);
									if (!NewParamName.IsNone())
									{
										ActionBase->AddPropertyBinding(NewParamName, Entry.Property);
									}
								}
							}
						}
					}
					return true;
				}
			}
		}
		else
		{
			return false;
		}

		for (const TPair<FName, TArray<FStalePropertyHandle>>& Pair : StaleBindings)
		{
			for (const FStalePropertyHandle& Entry : Pair.Value)
			{
				if (FSubsonicEventActionDefinition* Action = FindMutableAction(Entry.Action))
				{
					if (FSubsonicEventActionBase* ActionBase = Action->GetMutableActionBase())
					{
						ActionBase->RemovePropertyBinding(Entry.Property);
					}
				}
			}
		}

		return true;
#else // !WITH_EDITORONLY_DATA
		return false;
#endif // !WITH_EDITORONLY_DATA
	}

	void FSubsonicEventCollectionDefinition::UnbindParameter(FName ParamName)
	{
		for (TPair<FGameplayTag, FSubsonicEvent>& Pair : Events)
		{
			Pair.Value.UnbindParameter(ParamName);
		}
	}
#endif // WITH_EDITOR

	FSubsonicEventCollectionDefinition FSubsonicEventCollectionDefinition::Create(FName Name, TMap<FGameplayTag, FSubsonicEvent> InEvents, Audio::FDeviceId InDeviceId)
	{
		FSubsonicEventCollectionDefinition NewDefinition;
		NewDefinition.AssignId();

		const FCollectionHandle NewHandle
		{
	#if WITH_EDITORONLY_DATA
			.CollectionName = Name,
	#endif // WITH_EDITORONLY_DATA
			.CollectionId = NewDefinition.GetCollectionId(),
		};

		NewDefinition.DeviceId = InDeviceId;
		NewDefinition.Events = MoveTemp(InEvents);
		NewDefinition.Register(NewHandle, InDeviceId);
		return MoveTemp(NewDefinition);
	}

	FSubsonicEventCollectionDefinition::~FSubsonicEventCollectionDefinition()
	{
		Unregister();
		CollectionId = GetInvalidId();
		DeviceId = INDEX_NONE;
	}

#if WITH_EDITOR
	bool FSubsonicEventCollectionDefinition::ClearActions(const FEventHandle& InHandle)
	{
		if (FSubsonicEvent* Event = FindMutableEvent(InHandle))
		{
			Event->GetMutableActionCollection().Empty();
			return true;
		}

		return false;
	}

	void FSubsonicEventCollectionDefinition::ClearEvents()
	{
		Events.Empty();
	}

	bool FSubsonicEventCollectionDefinition::ContainsBinding(FName EventName, FName ParameterName) const
	{
		if (Parameters.FindPropertyDescByName(ParameterName) != nullptr)
		{
			return true;
		}

		if (const FSubsonicEvent* Event = FindEvent(EventName))
		{
			if (Event->GetParameters().FindPropertyDescByName(ParameterName) != nullptr)
			{
				return true;
			}
		}

		return false;
	}

	BindingUtils::FOnStaleBindingsDetected& FSubsonicEventCollectionDefinition::GetOnStaleBindingsDetectedDelegate()
	{
		return OnStaleBindingsDetected;
	}
#endif // WITH_EDITOR

	bool FSubsonicEventCollectionDefinition::ContainsEvent(const FGameplayTag& EventTag) const
	{
		return Events.Contains(EventTag);
	}

	bool FSubsonicEventCollectionDefinition::ContainsEvent(FName Name) const
	{
		const FGameplayTag EventTag = FGameplayTag::RequestGameplayTag(Name, CollectionPrivate::bErrorOnMissingTag);
		if (EventTag.GetTagName().IsValid() || Name.IsNone())
		{
			return ContainsEvent(EventTag);
		}

		return false;

	}

	bool FSubsonicEventCollectionDefinition::ContainsEvent(const FEventHandle& InHandle) const
	{
		if (InHandle.Collection.CollectionId == CollectionId)
		{
			return ContainsEvent(InHandle.EventName);
		}

		return false;
	}

	const FSubsonicEventActionDefinition* FSubsonicEventCollectionDefinition::FindAction(const FActionHandle& InHandle) const
	{
		if (const FSubsonicEvent* Event = FindEvent(InHandle.Event))
		{
			const TArray<FSubsonicEventActionDefinition>& Actions = Event->GetActionCollection();
			if (Actions.IsValidIndex(InHandle.Index))
			{
				return &Actions[InHandle.Index];
			}
		}

		return nullptr;
	}

	const FSubsonicEvent* FSubsonicEventCollectionDefinition::FindEvent(const FGameplayTag& EventTag) const
	{
		return Events.Find(EventTag);
	}

	const FSubsonicEvent* FSubsonicEventCollectionDefinition::FindEvent(FName Name) const
	{
		const FGameplayTag EventTag = FGameplayTag::RequestGameplayTag(Name, CollectionPrivate::bErrorOnMissingTag);
		return FindEvent(EventTag);
	}

	const FSubsonicEvent* FSubsonicEventCollectionDefinition::FindEvent(const FEventHandle& InHandle) const
	{
		if (InHandle.Collection.CollectionId == CollectionId)
		{
			return FindEvent(InHandle.EventName);
		}

		return nullptr;
	}

#if WITH_EDITOR
	FSubsonicEventActionDefinition* FSubsonicEventCollectionDefinition::FindMutableAction(const FActionHandle& InHandle)
	{
		if (FSubsonicEvent* Event = FindMutableEvent(InHandle.Event))
		{
			TArray<FSubsonicEventActionDefinition>& Actions = Event->ActionCollection;
			if (Actions.IsValidIndex(InHandle.Index))
			{
				return &Actions[InHandle.Index];
			}
		}

		return nullptr;
	}

	FSubsonicEvent* FSubsonicEventCollectionDefinition::FindMutableEvent(const FEventHandle& InHandle)
	{
		if (InHandle.Collection.CollectionId == GetCollectionId())
		{
			const FGameplayTag EventTag = FGameplayTag::RequestGameplayTag(InHandle.EventName, CollectionPrivate::bErrorOnMissingTag);
			if (EventTag.GetTagName().IsValid() || InHandle.EventName.IsNone())
			{
				return Events.Find(EventTag);
			}
		}

		return nullptr;
	}
#endif // WITH_EDITOR

	uint32 FSubsonicEventCollectionDefinition::GetCollectionId() const
	{
		return CollectionId;
	}

	const TMap<FGameplayTag, FSubsonicEvent>& FSubsonicEventCollectionDefinition::GetEvents() const
	{
		return Events;
	}

	uint32 FSubsonicEventCollectionDefinition::GetInvalidId()
	{
		return UE::Subsonic::Core::CollectionPrivate::InvalidId;
	}

#if WITH_EDITOR
	FName FSubsonicEventCollectionDefinition::GetEventsPropertyName()
	{
#if WITH_EDITORONLY_DATA
		return GET_MEMBER_NAME_CHECKED(FSubsonicEventCollectionDefinition, Events);
#else
		return { };
#endif // !WITH_EDITORONLY_DATA
	}

	FName FSubsonicEventCollectionDefinition::GetParametersPropertyName()
	{
#if WITH_EDITORONLY_DATA
		return GET_MEMBER_NAME_CHECKED(FSubsonicEventCollectionDefinition, Parameters);
#else
		return { };
#endif // !WITH_EDITORONLY_DATA
	}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	const FInstancedPropertyBag& FSubsonicEventCollectionDefinition::GetParameters() const
	{
		return Parameters;
	}
#endif // WITH_EDITORONLY_DATA

	bool FSubsonicEventCollectionDefinition::IsValid() const
	{
		return CollectionId != INDEX_NONE;
	}

	void FSubsonicEventCollectionDefinition::Register(const Core::FCollectionHandle& InHandle, Audio::FDeviceId InDeviceId)
	{
		if (IsValid())
		{
			DeviceId = InDeviceId;
			ISubsonicEventRegistry::GetChecked().OnCollectionRegistered(InHandle, DeviceId);
		}
	}

#if WITH_EDITOR
	bool FSubsonicEventCollectionDefinition::RemoveAction(const FActionHandle& InHandle)
	{
		if (FSubsonicEvent* Event = FindMutableEvent(InHandle.Event))
		{
			TArray<FSubsonicEventActionDefinition>& Actions = Event->GetMutableActionCollection();
			if (Actions.IsValidIndex(InHandle.Index))
			{
				Actions.RemoveAt(InHandle.Index);
				return true;
			}
		}

		return false;
	}

	bool FSubsonicEventCollectionDefinition::RemoveEvent(const FEventHandle& InHandle)
	{
		if (InHandle.Collection.CollectionId == CollectionId)
		{
			const FGameplayTag EventTag = FGameplayTag::RequestGameplayTag(InHandle.EventName, CollectionPrivate::bErrorOnMissingTag);
			if (EventTag.GetTagName().IsValid() || InHandle.EventName.IsNone())
			{
				return Events.Remove(EventTag) > 0;
			}
		}

		return false;
	}
#endif // WITH_EDITOR

	void FSubsonicEventCollectionDefinition::Unregister()
	{
		if (ISubsonicEventRegistry* EventRegistry = ISubsonicEventRegistry::Get(); EventRegistry && IsValid())
		{
			EventRegistry->OnCollectionUnregistered(FCollectionHandle { .CollectionId = CollectionId }, DeviceId);
		}

		CollectionId = CollectionPrivate::InvalidId;
		DeviceId = INDEX_NONE;
	}

} // namespace UE::Subsonic::Core
