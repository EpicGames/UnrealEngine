// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicEventSubscriberInterface.h"

#include "ISubsonicEventRegistry.h"
#include "SubsonicExecutor.h"
#include "SubsonicHandles.h"


namespace UE::Subsonic::Core
{
	void ISubsonicEventSubscriberInterface::OnCollectionRegistered(const FCollectionHandle& InCollection)
	{
	}

	void ISubsonicEventSubscriberInterface::OnCollectionUnregistered(const FCollectionHandle& InCollection)
	{
	}

	void ISubsonicEventSubscriberInterface::OnEventPreExecute(const FSubsonicExecutor& InExecutor, const FEventHandle& InHandle)
	{
	}

	void ISubsonicEventSubscriberInterface::OnEventPostExecute(const FSubsonicExecutor& InExecutor, const FEventHandle& InHandle)
	{
	}

	void ISubsonicEventSubscriberInterface::OnExecutorRegistered(const FSubsonicExecutor& InExecutor)
	{
	}

	void ISubsonicEventSubscriberInterface::OnExecutorUnregistered(const FSubsonicExecutor& InExecutor)
	{
	}

	void ISubsonicEventSubscriberInterface::Register()
	{
		ISubsonicEventRegistry::GetChecked().OnEventSubscriberRegistered(*this);
	}

	void ISubsonicEventSubscriberInterface::Unregister()
	{
		if (ISubsonicEventRegistry* Registry = ISubsonicEventRegistry::Get())
		{
			Registry->OnEventSubscriberUnregistered(*this);
		}
	}
} // namespace UE::Subsonic::Core