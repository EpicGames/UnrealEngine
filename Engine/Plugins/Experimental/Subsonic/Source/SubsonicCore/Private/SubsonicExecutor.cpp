// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicExecutor.h"

#include "ISubsonicEventRegistry.h"
#include "SubsonicCoreLog.h"
#include "SubsonicEventCollection.h"
#include "SubsonicHandles.h"


namespace UE::Subsonic::Core
{
	namespace ExecutorPrivate
	{
		constexpr uint32 InvalidId = static_cast<uint32>(INDEX_NONE);
		Audio::FDeviceId InvalidDeviceId = static_cast<Audio::FDeviceId>(INDEX_NONE);
	} // namespace ExecutorPrivate

	FSubsonicExecutor::FSubsonicExecutor()
		: DeviceId(ExecutorPrivate::InvalidDeviceId)
	{
	}

	FSubsonicExecutor::~FSubsonicExecutor()
	{
		Unregister();
	}

	TSharedRef<FSubsonicExecutor> FSubsonicExecutor::Create(Audio::FDeviceId InDeviceId, FSubsonicExecutor::FCollectionAccessorPtr&& CollectionPtr)
	{
		static uint32 MaxExecutorId = ExecutorPrivate::InvalidId;
		TSharedRef<FSubsonicExecutor> NewExecutor = MakeShared<FSubsonicExecutor>();
		NewExecutor->CollectionAccessor = MoveTemp(CollectionPtr);
		NewExecutor->ExecutorId = ++MaxExecutorId;
		NewExecutor->DeviceId = InDeviceId;
		NewExecutor->Register();
		return NewExecutor;
	}

	const FSubsonicEventCollectionDefinition* FSubsonicExecutor::GetCollection() const
	{
		if (CollectionAccessor.IsValid())
		{
			return CollectionAccessor->GetDefinition();
		}

		return nullptr;
	}

	FCollectionHandle FSubsonicExecutor::GetCollectionHandle() const
	{
		if (CollectionAccessor.IsValid())
		{
			return CollectionAccessor->GetHandle();
		}

		return FCollectionHandle();
	}

	Audio::FDeviceId FSubsonicExecutor::GetDeviceId() const
	{
		return DeviceId;
	}

	uint32 FSubsonicExecutor::GetId() const
	{
		return ExecutorId;
	}

	bool FSubsonicExecutor::ExecuteEvent(FName EventName) const
	{
		if (CollectionAccessor.IsValid())
		{
			if (const FSubsonicEventCollectionDefinition* Definition = CollectionAccessor->GetDefinition())
			{
				if (const FSubsonicEvent* Event = Definition->FindEvent(EventName))
				{
					if (Event->GetIsPublic())
					{
						const FCollectionHandle& CollectionHandle = CollectionAccessor->GetHandle();
						const FEventHandle EventHandle{ .Collection = CollectionHandle, .EventName = EventName };

						ISubsonicEventRegistry& Registry = ISubsonicEventRegistry::GetChecked();
						Registry.OnEventPreExecute(*this, EventHandle);
						Event->Execute(*this, EventHandle);
						Registry.OnEventPostExecute(*this, EventHandle);
						return true;
					}
				}

#if !NO_LOGGING
				const FString CollectionString =
#if WITH_EDITORONLY_DATA
					FString::Printf(TEXT(" in collection '%s'"), *CollectionAccessor->GetHandle().CollectionName.ToString());
#else // !WITH_EDITORONLY_DATA
					TEXT("");
#endif // !WITH_EDITORONLY_DATA

				UE_LOGF(LogSubsonic, Warning, "Failed to execute event '%ls'%ls: Event not found or accessible (not public).", *EventName.ToString(), *CollectionString);
#endif // !NO_LOGGING
			}
		}

		return false;
	}

	uint32 FSubsonicExecutor::GetInvalidId()
	{
		return ExecutorPrivate::InvalidId;
	}

	bool FSubsonicExecutor::IsValid() const
	{
		return ExecutorId != ExecutorPrivate::InvalidId;
	}

	void FSubsonicExecutor::Register()
	{
		ISubsonicEventRegistry::GetChecked().OnExecutorRegistered(*this);
	}

	void FSubsonicExecutor::Unregister()
	{
		if (IsValid())
		{
			ISubsonicEventRegistry::GetChecked().OnExecutorUnregistered(*this);
		}

		CollectionAccessor.Reset();
		Parameters.Reset();
		ExecutorId = ExecutorPrivate::InvalidId;
	}

	void FSubsonicExecutor::SetParameters(FSubsonicParameterStore InParams)
	{
		Parameters = MoveTemp(InParams);
	}

	const FSubsonicParameterStore& FSubsonicExecutor::GetParameters() const
	{
		return Parameters;
	}

	FString FSubsonicExecutor::ToString() const
	{
		return FString::Printf(TEXT("Id: %u, DeviceId: %u"), ExecutorId, DeviceId);
	}
} // namespace UE::Subsonic::Core
