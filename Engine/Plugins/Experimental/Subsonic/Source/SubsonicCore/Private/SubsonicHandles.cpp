// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicHandles.h"


namespace UE::Subsonic::Core
{
	namespace HandlesPrivate
	{
		uint32 InvalidHandleId = static_cast<uint32>(INDEX_NONE);
	} // namespace HandlesPrivate

	uint32 FCollectionHandle::GetInvalidId()
	{
		return HandlesPrivate::InvalidHandleId;
	}

	FString FCollectionHandle::ToString() const
	{
#if WITH_EDITORONLY_DATA
		return FString::Printf(TEXT("%s (Id: %u)"), *CollectionName.ToString(), CollectionId);
#else // !WITH_EDITORONLY_DATA
		return FString::Printf(TEXT("Id: %u"), CollectionId);
#endif // !WITH_EDITORONLY_DATA
	}

	FString FEventHandle::ToString() const
	{
		return FString::Printf(TEXT("%s/%s"), *Collection.ToString(), *EventName.ToString());
	}

	FString FActionHandle::ToString() const
	{
		return FString::Printf(TEXT("%s[%i]"), *Event.ToString(), Index);
	}
} // namespace UE::Subsonic::Core