// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicParameterStore.h"


namespace UE::Subsonic
{
	bool FSubsonicParameterStore::HasParameter(FName Name) const
	{
		const UPropertyBag* BagStruct = Bag.GetPropertyBagStruct();
		return BagStruct != nullptr && BagStruct->FindPropertyDescByName(Name) != nullptr;
	}

	void FSubsonicParameterStore::RemoveParameter(FName Name)
	{
		Bag.RemovePropertyByName(Name);
	}

	void FSubsonicParameterStore::Reset()
	{
		Bag = FInstancedPropertyBag{};
	}

	bool FSubsonicParameterStore::IsEmpty() const
	{
		return Bag.GetNumPropertiesInBag() == 0;
	}

	void FSubsonicParameterStore::MergeFrom(const FSubsonicParameterStore& Other)
	{
		const UPropertyBag* OtherBagStruct = Other.Bag.GetPropertyBagStruct();
		if (!OtherBagStruct)
		{
			return;
		}

		const TConstArrayView<FPropertyBagPropertyDesc> OtherDescs = OtherBagStruct->GetPropertyDescs();
		if (OtherDescs.IsEmpty())
		{
			return;
		}

		// Ensure all properties from Other exist in this bag, then overlay their values.
		Bag.AddProperties(OtherDescs);
		Bag.CopyMatchingValuesByName(Other.Bag);
	}
} // namespace UE::Subsonic
