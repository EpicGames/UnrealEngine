// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "Templates/UniquePtr.h"

#define UE_API SUBSONICCORE_API


namespace UE::Subsonic::Core
{
	// Forward Declarations
	struct FCollectionHandle;
	struct FSubsonicExecutor;
	struct FEventHandle;

	class ISubsonicEventSubscriberInterface;


	// Registry interface for events and their associated collections/executors.
	class ISubsonicEventRegistry
	{
	public:
		virtual ~ISubsonicEventRegistry() = default;

		UE_API static ISubsonicEventRegistry* Get();
		UE_API static ISubsonicEventRegistry& GetChecked();

		UE_API static void Initialize(TUniquePtr<ISubsonicEventRegistry>&& InRegistry);
		UE_API static void Deinitialize();

		virtual void OnCollectionRegistered(const FCollectionHandle& InHandle, Audio::FDeviceId DeviceId) = 0;
		virtual void OnCollectionUnregistered(const FCollectionHandle& InHandle, Audio::FDeviceId DeviceId) = 0;

		virtual void OnExecutorRegistered(const FSubsonicExecutor& InExecutor) = 0;
		virtual void OnExecutorUnregistered(const FSubsonicExecutor& InExecutor) = 0;

		virtual void OnEventPreExecute(const FSubsonicExecutor& InExecutor, const FEventHandle& InHandle) = 0;
		virtual void OnEventPostExecute(const FSubsonicExecutor& InExecutor, const FEventHandle& InHandle) = 0;

		virtual void OnEventSubscriberRegistered(ISubsonicEventSubscriberInterface& EventSubscriber) = 0;
		virtual void OnEventSubscriberUnregistered(ISubsonicEventSubscriberInterface& EventSubscriber) = 0;

	private:
		static TUniquePtr<ISubsonicEventRegistry> Instance;
	};
} // namespace UE::Subsonic::Core

#undef UE_API // SUBSONICCORE_API
