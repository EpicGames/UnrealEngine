// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateBase.h"
#include "Delegates/DelegateCombinations.h"
#include "SubsonicHandles.h"


#define UE_API SUBSONICCORE_API

namespace UE::Subsonic::Core
{
	namespace BindingUtils
	{
#if WITH_EDITOR
		// Response returned by OnStaleBindingsDetected to indicate whether to remove all stale
		// bindings (proceeding with the parameter structural change), revert the change, or rebind
		// stale bindings to a renamed parameter (where applicable).
		enum class EStaleBindingResponse : uint8
		{
			RemoveBindings,
			Revert,
			Rebind
		};

		// Represents minimal information required to locate a stale binding entry.
		struct FStalePropertyHandle
		{
			// Action containing stale binding.
			FActionHandle Action;

			// Name of property on action that is stale
			FName Property;
		};

		// Map of stale parameter name to action pointing to stale property
		using FStaleBindingsMap = TMap<FName, TArray<FStalePropertyHandle>>;

		// Fired when a structural parameter change (type or name change) creates stale editor bindings.
		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStaleBindingsDetected, const FStaleBindingsMap&, EStaleBindingResponse&);

		// Checks whether a property bag value can be safely copied into a destination (action) property.
		// Uses SameType for exact matching, with a fallback for object properties to allow subclass
		// assignment. Does not currently handle inner-type variance for container properties (TArray, etc.).
		UE_INTERNAL UE_API bool ArePropertiesBindingCompatible(const FProperty* FromBag, const FProperty* ToAction);

		// For a stale parameter name, finds a likely rename candidate in the current bags: a parameter
		// with a compatible type that is not already referenced by any existing binding on the affected
		// actions. Returns NAME_None if no unique candidate is found.
		UE_INTERNAL UE_API FName FindRenameCandidate(const FSubsonicEventCollectionDefinition& Definition, FName OldParamName, const TArray<FStalePropertyHandle>& StaleEntries);

		// Returns map of parameter names to stale binding entries for bindings that are no longer valid
		// (parameter removed or type no longer matches). Checks event-level bags first, then collection-level.
		UE_INTERNAL UE_API TMap<FName, TArray<FStalePropertyHandle>> FindStaleBindings(const FSubsonicEventCollectionDefinition& Definition, const FCollectionHandle& ParentHandle);

		// Prompts the user to handle stale bindings caused by a parameter rename or removal.
		// Detects rename candidates and shows either a 3-option (rename/remove/revert) or 2-option
		// (remove/revert) dialog. Sets OutResponse accordingly.
		UE_INTERNAL UE_API void PromptForStaleBindings(const FSubsonicEventCollectionDefinition& Definition, const FStaleBindingsMap& StaleBindings, EStaleBindingResponse& OutResponse);
#endif // WITH_EDITOR
	} // namespace BindingUtils
} // namespace UE::Subsonic::Core

#undef UE_API // SUBSONICCORE_API
