// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::Subsonic::BuiltInParameters
{
	extern SUBSONICCORE_API const FName Volume;
	extern SUBSONICCORE_API const FName PitchShift;
	extern SUBSONICCORE_API const FName HighpassCutoff;
	extern SUBSONICCORE_API const FName LowpassCutoff;
	extern SUBSONICCORE_API const FName FadeOutTime;

} // namespace UE::Subsonic::BuiltInParameters
