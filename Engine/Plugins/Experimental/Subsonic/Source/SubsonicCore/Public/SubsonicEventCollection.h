// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "GameplayTagContainer.h"
#include "NativeGameplayTags.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"
#include "SubsonicBindingUtils.h"
#include "SubsonicHandles.h"
#include "UObject/NameTypes.h"

#include "SubsonicEventCollection.generated.h"

// Forward Declarations
class FProperty;

#define UE_API SUBSONICCORE_API

namespace UE::Subsonic::Core
{
	// Forward Declarations
	struct FSubsonicExecutor;

	UE_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(TAG_SubsonicCore)
	UE_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(TAG_SubsonicCore_Event_Play)
	UE_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(TAG_SubsonicCore_Event_Stop)

	USTRUCT()
	struct FSubsonicEventActionBoundProperties
	{
		GENERATED_BODY()

		UPROPERTY()
		TArray<FName> Properties;
	};

	USTRUCT(MinimalAPI, BlueprintType, meta = (Abstract, Hidden))
	struct FSubsonicEventActionBase
	{
		GENERATED_BODY()

		virtual ~FSubsonicEventActionBase() = default;

#if WITH_EDITOR
		UE_INTERNAL UE_API const TMap<FName, FSubsonicEventActionBoundProperties>& GetBindings() const;
		UE_INTERNAL UE_API static FName GetBindingsParameterName();

		// Computes a reasonable default base identifiable name from the provided owning event handle.
		UE_API static FName GenerateDefaultBaseIdentifier(const FEventHandle& OwnerHandle);

		// Returns display info that is custom to the given action instance to be shown
		// adjacent to the action, for example, in the Subsonic Event Editor action tree.
		UE_API virtual FText GetDisplayInfo() const;

		// Initializes default property names on a newly created action config.
		// Computes a base name from context, then delegates to the virtual InitializeDefaultName.
		UE_API static void InitializeDefaultActionName(TInstancedStruct<FSubsonicEventActionBase>& ActionConfig, const FEventHandle& OwnerHandle);

		// Virtual hook for per-action-type customization of default property names.
		// Called by InitializeDefaultActionName with the computed base name.
		// Base implementation sets the first FNameProperty named "Name" to BaseName via reflection.
		UE_API virtual void InitializeDefaultName(const UScriptStruct* ActionStruct, FName BaseName);
#endif // WITH_EDITOR

	public:
#if WITH_EDITOR
		// Returns the parameter name that the given action property is currently bound to, or NAME_None if unbound.
		UE_API FName GetBoundParameterForProperty(FName PropertyName) const;

		// Binds an action property to a named parameter, replacing any previous binding for that property.
		UE_API void AddPropertyBinding(FName ParameterName, FName PropertyName);

		// Removes all property bindings associated with the parameter of the given name.
		// Returns true if binding was found and removed, false if not.
		UE_API bool RemoveAllPropertyBindingsToParameter(FName ParameterName);

		// Removes any binding for the given action property.
		UE_API void RemovePropertyBinding(FName PropertyName);
#endif // WITH_EDITOR

	protected:
		virtual void Execute(const FSubsonicExecutor& InExecutor, const FActionHandle& InHandle) const { }

#if WITH_EDITORONLY_DATA
		UPROPERTY()
		TMap<FName, FSubsonicEventActionBoundProperties> Bindings;
#endif // WITH_EDITORONLY_DATA

		friend struct FSubsonicEvent;
	};

	// Action definition iteratively executed per Subsonic event.
	USTRUCT(MinimalAPI, BlueprintType)
	struct FSubsonicEventActionDefinition
	{
		GENERATED_BODY()

		UE_API const TInstancedStruct<FSubsonicEventActionBase>& GetAction() const;

#if WITH_EDITOR
		UE_API void SetAction(TInstancedStruct<FSubsonicEventActionBase> NewAction);

		// Returns a mutable pointer to the underlying action base, or nullptr if no action is set.
		UE_INTERNAL UE_API FSubsonicEventActionBase* GetMutableActionBase();
#endif // WITH_EDITOR
 
	private:
		friend struct FSubsonicEvent;

		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Actions", meta = (AllowPrivateAccess))
		TInstancedStruct<FSubsonicEventActionBase> Action;
	};

	USTRUCT(MinimalAPI, BlueprintType)
	struct FSubsonicEvent
	{
		GENERATED_BODY()

		virtual ~FSubsonicEvent() = default;

#if WITH_EDITOR
		UE_INTERNAL UE_API static FName GetActionCollectionPropertyName();
		UE_INTERNAL UE_API static FName GetParametersPropertyName();
#endif // WITH_EDITOR

		UE_API const TArray<FSubsonicEventActionDefinition>& GetActionCollection() const;

#if WITH_EDITORONLY_DATA
		UE_API const FInstancedPropertyBag& GetParameters() const;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
		UE_API bool GetAutoAudition() const;
#endif // WITH_EDITOR

		UE_API bool GetIsPublic() const;

#if WITH_EDITOR
		UE_INTERNAL UE_API TArray<FSubsonicEventActionDefinition>& GetMutableActionCollection();
		UE_INTERNAL UE_API void SetAutoAudition(bool bInAutoAudition);
		UE_INTERNAL UE_API void SetIsPublic(bool bInIsPublic);
#endif // WITH_EDITOR

	private:
		friend struct FSubsonicEventCollectionDefinition;
		friend struct FSubsonicExecutor;

#if WITH_EDITOR
		UE_INTERNAL void BindParameter(const FInstancedPropertyBag& InSourceParameters, FName Name);
		UE_INTERNAL void UnbindParameter(FName Name);
#endif // WITH_EDITOR

	private:
		// Called by executor, iterates through all actions on the given event definition.
		// Asserts if the provided executor and handle have mismatched collection contexts.
		UE_INTERNAL void Execute(const FSubsonicExecutor& InExecutor, const FEventHandle& InHandle) const;

		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Event", meta = (AllowPrivateAccess))
		bool bIsPublic = true;

#if WITH_EDITORONLY_DATA
		// If true, execute this event as soon as an audition executor is created.
		UPROPERTY(EditAnywhere, Transient, Category = "Event", meta = (EditCondition = bIsPublic))
		bool bAutoAudition = false;

		UPROPERTY(EditAnywhere, Category = "Event")
		FInstancedPropertyBag Parameters;
#endif // WITH_EDITORONLY_DATA

		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Actions", meta = (AllowPrivateAccess))
		TArray<FSubsonicEventActionDefinition> ActionCollection;
	};


	USTRUCT(MinimalAPI, BlueprintType)
	struct FSubsonicEventCollectionDefinition final
	{
		GENERATED_BODY()

		// Generates empty, invalid event collection. Required to be public
		// to support UObject reflection. To produce a valid definition natively
		// without UObject support, call static Create function below.
		FSubsonicEventCollectionDefinition() = default;

		FSubsonicEventCollectionDefinition(FSubsonicEventCollectionDefinition&& Definition);
		FSubsonicEventCollectionDefinition(const FSubsonicEventCollectionDefinition& Definition) = delete;

		UE_API FSubsonicEventCollectionDefinition& operator=(FSubsonicEventCollectionDefinition&& Definition);
		FSubsonicEventCollectionDefinition& operator=(const FSubsonicEventCollectionDefinition& Definition) = delete;

		UE_API ~FSubsonicEventCollectionDefinition();

		// Create a collection definition with a valid transient Id for tracking
		// within the Subsonic System. Used to create and register definitions
		// outside of UObject reflection.
		static FSubsonicEventCollectionDefinition Create(FName Name, TMap<FGameplayTag, FSubsonicEvent> InEvents, Audio::FDeviceId DeviceId);

#if WITH_EDITOR
		// Adds action to event with handle if it exists. If insert index set to less than 0 (default), adds to end of collection array.
		UE_INTERNAL UE_API FSubsonicEventActionDefinition* AddAction(const FEventHandle& InHandle, int32 InsertIndex = INDEX_NONE);

		// Adds event with given tag if does not yet exists. Returns pointer to new event if added, null if none added.
		UE_INTERNAL UE_API FSubsonicEvent* AddEvent(const FGameplayTag& InTag = { }, FSubsonicEvent NewEvent = {});
#endif // WITH_EDITOR

		// Assigns a universal CollectionId to the given collection.
		UE_API UE_INTERNAL void AssignId();

#if WITH_EDITOR
		// Clears all actions under the given event
		UE_INTERNAL UE_API bool ClearActions(const FEventHandle& InHandle);

		// Removes all events in the given collection.
		UE_INTERNAL UE_API void ClearEvents();

		// Returns true if binding exists with the given parameter name on either the collection or event with
		// the given name, false if not. Also returns false if event with the given name is not found.
		UE_INTERNAL UE_API bool ContainsBinding(FName EventName, FName ParameterName) const;

		// Fired when a structural parameter change creates stale editor bindings.
		// The handler should show appropriate UI and return the desired response.
		// If unbound, stale bindings are removed automatically.
		UE_INTERNAL UE_API BindingUtils::FOnStaleBindingsDetected& GetOnStaleBindingsDetectedDelegate();
#endif // WITH_EDITOR

		UE_API bool ContainsEvent(FName Name) const;
		UE_API bool ContainsEvent(const FGameplayTag& EventTag) const;
		UE_API bool ContainsEvent(const FEventHandle& InHandle) const;

		// Returns pointer to action at handle if it exists
		UE_API const FSubsonicEventActionDefinition* FindAction(const FActionHandle& InHandle) const;

		// Returns event with the given name.
		UE_API const FSubsonicEvent* FindEvent(FName Name) const;
		UE_API const FSubsonicEvent* FindEvent(const FGameplayTag& EventTag) const;
		UE_API const FSubsonicEvent* FindEvent(const FEventHandle& InHandle) const;

#if WITH_EDITOR
		// Returns pointer to action at handle if it exists
		UE_INTERNAL UE_API FSubsonicEventActionDefinition* FindMutableAction(const FActionHandle& InHandle);

		// Returns mutable pointer to event at handle if it exists
		UE_INTERNAL UE_API FSubsonicEvent* FindMutableEvent(const FEventHandle& InHandle);
#endif // WITH_EDITOR

		UE_API uint32 GetCollectionId() const;

		UE_API const TMap<FGameplayTag, FSubsonicEvent>& GetEvents() const;

		// Returns Id of invalid (default constructed or unregistered/reset collection).
		UE_API static uint32 GetInvalidId();

#if WITH_EDITOR
		UE_API static FName GetEventsPropertyName();
		UE_API static FName GetParametersPropertyName();
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
		UE_API const FInstancedPropertyBag& GetParameters() const;
#endif // WITH_EDITORONLY_DATA

		UE_API bool IsValid() const;

		// Register's the given collection with the SubsonicEventRegistry.
		UE_API UE_INTERNAL void Register(const Core::FCollectionHandle& InHandle, Audio::FDeviceId InDeviceId);

#if WITH_EDITOR
		// Removes action at given handle and returns if successful.
		UE_INTERNAL UE_API bool RemoveAction(const FActionHandle& InHandle);

		// Removes event at given handle and returns if successful.
		UE_INTERNAL UE_API bool RemoveEvent(const FEventHandle& InHandle);

		// Removes all stale property bindings. Defined as stale if type or
		// name of mapped property does not match that of target parameter.
		// If bPromptRemoval is set, delegate is executed enabling editors to
		// prompt user to cancel removal. Returns whether or not removal succeeded.
		UE_INTERNAL UE_API bool RemoveStaleBindings(const FCollectionHandle& ParentHandle, bool bPromptRemoval = false);

		// Binds all parameters across all events in the collection.
		UE_INTERNAL UE_API void BindAllParameters();
#endif // WITH_EDITOR

		// Unregisters collection, leaving it in clear, invalid state.
		UE_API UE_INTERNAL void Unregister();

	private:
#if WITH_EDITOR
		BindingUtils::FOnStaleBindingsDetected OnStaleBindingsDetected;

		UE_INTERNAL void BindParameter(FName Parameter);
		UE_INTERNAL void UnbindParameter(FName Parameter);

		// Removes all editor bindings to the given parameter name from all actions
		// in all events. Returns whether or not any bindings were removed.
		UE_INTERNAL UE_API bool RemoveAllPropertyBindingsToParameter(FName ParamName);
#endif // WITH_EDITOR

		// Unique Collection Id assigned post
		// initialization of struct properties
		// or via static initializer
		// Independent from UObject identifier to
		// allow for transient construction without
		// having to utilize UObject wrapper.
		uint32 CollectionId = INDEX_NONE;

		// Device associated with the given event collection
		Audio::FDeviceId DeviceId = INDEX_NONE;

		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Events", meta = (AllowPrivateAccess))
		TMap<FGameplayTag, FSubsonicEvent> Events;

#if WITH_EDITORONLY_DATA
		UPROPERTY(EditAnywhere, Category = "Parameters")
		FInstancedPropertyBag Parameters;
#endif // WITH_EDITORONLY_DATA
	};

} // namespace UE::Subsonic::Core

// Required to be in global namespace still for UHT compatibility with namespaced Unreal reflected types
template <>
struct TStructOpsTypeTraits<UE::Subsonic::Core::FSubsonicEventCollectionDefinition>
	: public TStructOpsTypeTraitsBase2<UE::Subsonic::Core::FSubsonicEventCollectionDefinition>
{
	enum { WithCopy = false };
};
#undef UE_API // SUBSONICCORE_API
