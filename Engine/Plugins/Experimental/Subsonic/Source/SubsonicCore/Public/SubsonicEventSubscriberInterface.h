// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Interface.h"

#include "SubsonicEventSubscriberInterface.generated.h"

#define UE_API SUBSONICCORE_API

namespace UE::Subsonic::Core
{
	// Forward Declarations
	struct FCollectionHandle;
	struct FEventHandle;
	struct FSubsonicExecutor;

	// Interface for all subscribers that respond to Subsonic events.
	UINTERFACE(MinimalAPI, BlueprintType, meta = (DisplayName = "Subsonic Event Subscriber Interface", CannotImplementInterfaceInBlueprint))
	class USubsonicEventSubscriberInterface : public UInterface
	{
		GENERATED_BODY()
	};

	class ISubsonicEventSubscriberInterface : public IInterface
	{
		GENERATED_BODY()

	public:
		UE_API virtual void OnCollectionRegistered(const FCollectionHandle& InCollection);
		UE_API virtual void OnCollectionUnregistered(const FCollectionHandle& InCollection);

		UE_API virtual void OnEventPreExecute(const FSubsonicExecutor& InExecutor, const FEventHandle& InHandle);
		UE_API virtual void OnEventPostExecute(const FSubsonicExecutor& InExecutor, const FEventHandle& InHandle);

		UE_API virtual void OnExecutorRegistered(const FSubsonicExecutor& InExecutor);
		UE_API virtual void OnExecutorUnregistered(const FSubsonicExecutor& InExecutor);

	protected:
		UE_API virtual void Register();
		UE_API virtual void Unregister();
	};
} // namespace UE::Subsonic::Core

#undef UE_API // SUBSONICCORE_API