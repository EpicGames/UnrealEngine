// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "HAL/Platform.h"
#include "SubsonicHandles.h"
#include "SubsonicParameterStore.h"
#include "Templates/SharedPointer.h"
#include "Templates/SharedPointerInternals.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#include "SubsonicExecutor.generated.h"


// Forward Declarations
struct FSubsonicEvent;
struct FSubsonicEventCollectionDefinition;

#define UE_API SUBSONICCORE_API


UENUM(BlueprintType)
enum class ESubsonicExecutionScope : uint8
{
	Global,
	Executor
};


namespace UE::Subsonic::Core
{
	// Execution object that tracks explicit execution context for any caller of events related to a given collection.
	// RAII design that automatically registers and unregisters with the Subsonic Event Registry, and by extension,
	// any implemented and active subscribers. This enables subscribers to track and manage data related to
	// event requests from a given executor instance in a data-oriented fashion.
	struct FSubsonicExecutor final : public TSharedFromThis<FSubsonicExecutor>
	{
	private:
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;

		FSubsonicExecutor();
		FSubsonicExecutor(const FSubsonicExecutor&) = delete;
		FSubsonicExecutor(FSubsonicExecutor&&) = delete;
		FSubsonicExecutor& operator=(const FSubsonicExecutor& InOther) = delete;
		FSubsonicExecutor& operator=(FSubsonicExecutor&& InOther) = delete;

	public:
		~FSubsonicExecutor();

		struct ICollectionAccessor
		{
			virtual ~ICollectionAccessor() = default;

			virtual const FSubsonicEventCollectionDefinition* GetDefinition() const = 0;
			virtual FCollectionHandle GetHandle() const = 0;
		};
		using FCollectionAccessorPtr = TUniquePtr<ICollectionAccessor>;

		UE_API static TSharedRef<FSubsonicExecutor> Create(Audio::FDeviceId InDeviceId, FCollectionAccessorPtr&& CollectionPtr);

		UE_API bool ExecuteEvent(FName EventName) const;

		UE_API static uint32 GetInvalidId();

		UE_API const FSubsonicEventCollectionDefinition* GetCollection() const;
		UE_API FCollectionHandle GetCollectionHandle() const;
		UE_API Audio::FDeviceId GetDeviceId() const;
		UE_API uint32 GetId() const;

		UE_API bool IsValid() const;

		UE_API FString ToString() const;

		UE_API void Unregister();

		// Sets trigger-time parameters on this executor. Actions will merge these with their
		// authored parameters when Execute() is called. Cleared on Unregister().
		UE_API void SetParameters(FSubsonicParameterStore InParams);

		// Returns the current trigger-time parameter store.
		UE_API const FSubsonicParameterStore& GetParameters() const;

	private:
		void Register();

		FCollectionAccessorPtr CollectionAccessor;

		Audio::FDeviceId DeviceId;

		uint32 ExecutorId = GetInvalidId();

		// Trigger-time parameters stored here before ExecuteEvent is called.
		FSubsonicParameterStore Parameters;
	};

	// Key consisting of executor ID and Audio DeviceId (holds CollectionHandle in builds with logging
	// enabled for debugging). Provides easily readable, hashable key while not providing direct executor 
	// ccess, enabling subscribers to organize data based in hashed executor scope related to execution context.
	struct FExecutorScopeKey
	{
		uint32 ExecutorId = FSubsonicExecutor::GetInvalidId();
		Audio::FDeviceId DeviceId = INDEX_NONE;

#if !NO_LOGGING
		FCollectionHandle CollectionHandle;
#endif // !NO_LOGGING

		FExecutorScopeKey() = default;
		FExecutorScopeKey(const FSubsonicExecutor& InExecutor)
			: ExecutorId(InExecutor.GetId())
			, DeviceId(InExecutor.GetDeviceId())
#if !NO_LOGGING
			, CollectionHandle(InExecutor.GetCollectionHandle())
#endif // !NO_LOGGING
		{
		}

		friend uint32 GetTypeHash(const FExecutorScopeKey& Key)
		{
			return HashCombineFast(Key.ExecutorId, GetTypeHash(Key.DeviceId));
		}

		friend bool operator==(const FExecutorScopeKey& A, const FExecutorScopeKey& B)
		{
			return A.ExecutorId == B.ExecutorId && A.DeviceId == B.DeviceId;
		}

		FString ToString() const
		{
#if NO_LOGGING
			return FString::Printf(TEXT("ExecId: %u, AudioDevice Id: %u"), ExecutorId, DeviceId);
#else // !NO_LOGGING
			return FString::Printf(TEXT("%s [ExecId: %u, AudioDevice Id: %u]"), *CollectionHandle.ToString(), ExecutorId, DeviceId);
#endif // !NO_LOGGING
		}
	};
} // namespace UE::Subsonic::Core

#undef UE_API // SUBSONICCORE_API
