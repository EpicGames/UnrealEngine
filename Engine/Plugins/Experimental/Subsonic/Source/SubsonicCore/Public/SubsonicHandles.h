// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreDefines.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"

#define UE_API SUBSONICCORE_API


namespace UE::Subsonic::Core
{
	// Forward Declarations
	struct FSubsonicEvent;
	struct FSubsonicEventCollectionDefinition;

	struct FCollectionHandle
	{
#if WITH_EDITORONLY_DATA
		// Owning Collection definition's name (for action initialization and logging/debugging)
		FName CollectionName;
#endif // WITH_EDITORONLY_DATA

		// Returns invalid Id of a handle
		UE_API static uint32 GetInvalidId();

		// Owning Collection's instance id
		uint32 CollectionId = GetInvalidId();

		friend uint32 GetTypeHash(const FCollectionHandle& Handle)
		{
			return Handle.CollectionId;
		}

		friend bool operator==(const FCollectionHandle& A, const FCollectionHandle& B)
		{
			return A.CollectionId == B.CollectionId;
		}

		UE_API FString ToString() const;
	};

	struct FEventHandle
	{
		// Parent collection handle
		FCollectionHandle Collection;

		// Owning event name
		FName EventName;

		friend uint32 GetTypeHash(const FEventHandle& Handle)
		{
			return HashCombineFast(GetTypeHash(Handle.Collection), GetTypeHash(Handle.EventName));
		}

		friend bool operator==(const FEventHandle& A, const FEventHandle& B)
		{
			return A.Collection == B.Collection && A.EventName == B.EventName;
		}

		UE_API FString ToString() const;
	};

	struct FActionHandle
	{
		// Parent event handle
		FEventHandle Event;

		// Owning action's index within a given event
		int32 Index = INDEX_NONE;

		friend uint32 GetTypeHash(const FActionHandle& Handle)
		{
			return HashCombineFast(GetTypeHash(Handle.Event), GetTypeHash(Handle.Index));
		}

		friend bool operator==(const FActionHandle& A, const FActionHandle& B)
		{
			return A.Event == B.Event && A.Index == B.Index;
		}

		UE_API FString ToString() const;
	};
} // namespace UE::Subsonic::Core

#undef UE_API // SUBSONICCORE_API
