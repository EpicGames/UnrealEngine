// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"

#include "SubsonicParameterStore.generated.h"

#define UE_API SUBSONICCORE_API


namespace UE::Subsonic
{
	// A parameter store for Subsonic runtime parameters, backed by an FInstancedPropertyBag.
	// Priority layers (lowest to highest): Authored < TriggerTime < Runtime.
	USTRUCT(BlueprintType)
	struct FSubsonicParameterStore
	{
		GENERATED_BODY()

		UE_API bool HasParameter(FName Name) const;
		UE_API void RemoveParameter(FName Name);
		UE_API void Reset();
		UE_API bool IsEmpty() const;

		// Overlay: properties in Other are added to this store and their values overwrite any
		// existing values with the same name.
		UE_API void MergeFrom(const FSubsonicParameterStore& Other);

		UPROPERTY(EditAnywhere, Category = "Parameters")
		FInstancedPropertyBag Bag;
	};
} // namespace UE::Subsonic

#undef UE_API
