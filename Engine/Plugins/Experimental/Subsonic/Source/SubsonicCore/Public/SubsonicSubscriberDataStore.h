// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/ForEach.h"
#include "Containers/Map.h"
#include "SubsonicExecutor.h"
#include "Templates/Function.h"


namespace UE::Subsonic::Core
{
	// Generalized scope- and executor-keyed data store. Stores a DataType instance per scope level
	// (Global, Executor) and provides convenience accessors, executor tracking, and iteration.
	template <typename DataType>
	class TSubscriberDataStore
	{
	private:
		using FDataMap = TMap<FName, DataType>;
		using FNameDataPair = TPair<FName, DataType>;
		using FKeyDataMapPair = TPair<FExecutorScopeKey, FDataMap>;

		FDataMap GlobalStore;
		TMap<FExecutorScopeKey, FDataMap> ExecutorDataStore;

	public:
		// Empties all scoped stores
		void Empty()
		{
			GlobalStore.Empty();
			ExecutorDataStore.Empty();
		}

		// Empties all scoped stores, running the provided release function
		// on all data values prior to destruction.
		void Empty(TFunctionRef<void(FName, DataType&)> ReleaseFunc)
		{
			ForEach(ReleaseFunc);

			auto IterPairs = [this, &ReleaseFunc](FKeyDataMapPair& Pair)
			{
				ForEach(Pair.Key, ReleaseFunc);
			};
			Algo::ForEach(ExecutorDataStore, IterPairs);
			Empty();
		}

		// Finds associated data with a given global identifier
		DataType* Find(FName GlobalName)
		{
			return GlobalStore.Find(GlobalName);
		}

		// Finds data associated with a given global identifier
		const DataType* Find(FName GlobalName) const
		{
			return GlobalStore.Find(GlobalName);
		}

		// Finds data associated with a given executor and name
		DataType* Find(const FExecutorScopeKey& Key, FName Name)
		{
			if (FDataMap* Map = ExecutorDataStore.Find(Key))
			{
				return Map->Find(Name);
			}

			return nullptr;
		}

		// Finds data associated with a given executor and name
		const DataType* Find(const FExecutorScopeKey& Key, FName Name) const
		{
			if (const FDataMap* Map = ExecutorDataStore.Find(Key))
			{
				return Map->Find(Name);
			}

			return nullptr;
		}


		// Finds or adds data associated with a given global identifier
		DataType& FindOrAdd(FName Name)
		{
			return GlobalStore.FindOrAdd(Name);
		}

		// Finds or adds data associated with a given executor & name.
		DataType& FindOrAdd(const FExecutorScopeKey& Key, FName Name)
		{
			FDataMap& Map = ExecutorDataStore.FindOrAdd(Key);
			return Map.FindOrAdd(Name);
		}

		// Iterate on all data values associated with the given key with all names.
		void ForEach(const FExecutorScopeKey& InKey, TFunctionRef<void(FName, DataType&)> InFunc)
		{
			if (FDataMap* Map = ExecutorDataStore.Find(InKey))
			{
				auto FuncOnValues = [&InFunc](FNameDataPair& Pair) { InFunc(Pair.Key, Pair.Value); };
				Algo::ForEach(*Map, FuncOnValues);
			}
		}

		// Iterate on all data values associated with the given key with all names.
		void ForEach(const FExecutorScopeKey& InKey, TFunctionRef<void(FName, DataType&)> InFunc) const
		{
			if (const FDataMap* Map = ExecutorDataStore.Find(InKey))
			{
				auto FuncOnValues = [&InFunc](const FNameDataPair& Pair) { InFunc(Pair.Key, Pair.Value); };
				Algo::ForEach(*Map, FuncOnValues);
			}
		}

		// Iterate on all executor-scoped data maps with their associated keys.
		void ForEach(TFunctionRef<void(const FExecutorScopeKey&, const FName&, DataType&)> InFunc)
		{
			auto FuncOnPairs = [&InFunc](FKeyDataMapPair& MapPair)
			{
				Algo::ForEach(MapPair.Value, [&](TPair<FName, DataType>& DataPair)
				{
					InFunc(MapPair.Key, DataPair.Key, DataPair.Value);
				});
			};
			Algo::ForEach(ExecutorDataStore, FuncOnPairs);
		}

		// Iterate on all executor-scoped data with their associated keys.
		void ForEach(TFunctionRef<void(const FExecutorScopeKey&, const FName&, const DataType&)> InFunc) const
		{
			auto FuncOnPairs = [&InFunc](const FKeyDataMapPair& MapPair)
			{
				Algo::ForEach(MapPair.Value, [&](const TPair<FName, DataType>& DataPair)
				{
					InFunc(MapPair.Key, DataPair.Key, DataPair.Value);
				});
			};
			Algo::ForEach(ExecutorDataStore, FuncOnPairs);
		}

		// Iterate on all values at global scope with all names.
		void ForEach(TFunctionRef<void(FName, DataType&)> InFunc)
		{
			auto FuncOnValues = [&InFunc](FNameDataPair& Pair) { InFunc(Pair.Key, Pair.Value); };
			Algo::ForEach(GlobalStore, FuncOnValues);
		}

		// Iterate on all values at global scope with all names.
		void ForEach(TFunctionRef<void(FName, const DataType&)> InFunc) const
		{
			auto FuncOnValues = [&InFunc](const FNameDataPair& Pair) { InFunc(Pair.Key, Pair.Value); };
			Algo::ForEach(GlobalStore, FuncOnValues);
		}

		// Removes the named data at the global scope level.
		int32 Remove(FName Name)
		{
			return GlobalStore.Remove(Name);
		}

		// Removes data associated with the given name at the executor scope level.
		int32 Remove(const FExecutorScopeKey& Key, FName Name)
		{
			int32 NumRemoved = 0;
			if (FDataMap* Data = ExecutorDataStore.Find(Key))
			{
				NumRemoved = Data->Remove(Name);
				if (Data->IsEmpty())
				{
					ExecutorDataStore.Remove(Key);
				}
			}

			return NumRemoved;
		}

		// Removes all named data at the executor scope level.
		int32 Remove(const FExecutorScopeKey& Key)
		{
			return ExecutorDataStore.Remove(Key);
		}
	};
} // namespace UE::Subsonic::Core
