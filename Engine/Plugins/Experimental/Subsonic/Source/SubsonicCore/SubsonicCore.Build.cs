// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SubsonicCore : ModuleRules
{
	public SubsonicCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioMixerCore",
				"Core",
				"GameplayTags",
				"PropertyBindingUtils"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
			}
		);

		bAllowUETypesInNamespaces = true;
	}
}
