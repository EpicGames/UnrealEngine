// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicAssetDefinitions.h"

#include "Styling/SlateStyleRegistry.h"
#include "SubsonicEventCollectionEditor.h"
#include "SubsonicEventCollectionObjects.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubsonicAssetDefinitions)

#define LOCTEXT_NAMESPACE "SubsonicEditor"


namespace UE::Subsonic
{
	namespace AssetDefinitionsPrivate
	{
		const FSlateBrush& GetSlateBrushSafe(FName InName)
		{
			const ISlateStyle* Style = FSlateStyleRegistry::FindSlateStyle("SubsonicStyle");
			if (ensureMsgf(Style, TEXT("Missing slate style 'SubsonicStyle'")))
			{
				const FSlateBrush* Brush = Style->GetBrush(InName);
				if (ensureMsgf(Brush, TEXT("Missing brush '%s'"), *InName.ToString()))
				{
					return *Brush;
				}
			}

			if (const FSlateBrush* NoBrush = FAppStyle::GetBrush("NoBrush"))
			{
				return *NoBrush;
			}

			static const FSlateBrush NullBrush;
			return NullBrush;
		}

		const FSlateBrush* GetClassBrush(const FAssetData& InAssetData, FName InClassName, bool bIsThumbnail = false)
		{
			FString BrushName = FString::Printf(TEXT("SubsonicEditor.%s"), *InClassName.ToString());
			BrushName += bIsThumbnail ? TEXT(".Thumbnail") : TEXT(".Icon");
			return &GetSlateBrushSafe(*BrushName);
		}
	} // namespace AssetDefinitionsPrivate

	FLinearColor UAssetDefinition_SubsonicEventCollection::GetAssetColor() const
	{
 		if (const ISlateStyle* SubsonicStyle = FSlateStyleRegistry::FindSlateStyle("SubsonicStyle"))
 		{
 			return SubsonicStyle->GetColor("SubsonicEventCollection.Color").ToFColorSRGB();
 		}
 
 		return FColor::White;
	}

	TSoftClassPtr<UObject> UAssetDefinition_SubsonicEventCollection::GetAssetClass() const
	{
		return USubsonicEventCollection::StaticClass();
	}

	TConstArrayView<FAssetCategoryPath> UAssetDefinition_SubsonicEventCollection::GetAssetCategories() const
	{
		static const auto Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Audio, LOCTEXT("AssetSoundSubsonicSubMenu", "Experimental"))
			};
		return Categories;
	}

	EAssetCommandResult UAssetDefinition_SubsonicEventCollection::OpenAssets(const FAssetOpenArgs& OpenArgs) const
	{
		using namespace Editor;

		for (const FAssetData& AssetData : OpenArgs.Assets)
		{
			if (USubsonicEventCollection* Collection = Cast<USubsonicEventCollection>(AssetData.GetAsset()))
			{
				TSharedRef<FEventCollectionEditor> NewEditor = MakeShared<FEventCollectionEditor>();
				NewEditor->Init(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, *Collection);
			}
		}

		return EAssetCommandResult::Handled;
	}

	const FSlateBrush* UAssetDefinition_SubsonicEventCollection::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
	{
		if (const FSlateBrush* ClassBrush = AssetDefinitionsPrivate::GetClassBrush(InAssetData, InClassName, true /* bIsThumbnail */))
		{
			return ClassBrush;
		}

		return Super::GetThumbnailBrush(InAssetData, InClassName);
	}

	const FSlateBrush* UAssetDefinition_SubsonicEventCollection::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
	{
		if (const FSlateBrush* ClassBrush = AssetDefinitionsPrivate::GetClassBrush(InAssetData, InClassName))
		{
			return ClassBrush;
		}

		return Super::GetIconBrush(InAssetData, InClassName);
	}

	bool UAssetDefinition_SubsonicEventCollection::GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const
	{
		return Super::GetThumbnailActionOverlay(InAssetData, OutActionOverlayInfo);
	}

	EAssetCommandResult UAssetDefinition_SubsonicEventCollection::ActivateAssets(const FAssetActivateArgs& ActivateArgs) const
	{
		return Super::ActivateAssets(ActivateArgs);
	}

	void UAssetDefinition_SubsonicEventCollection::GetAssetActionButtonExtensions(const FAssetData& InAssetData, TArray<FAssetButtonActionExtension>& OutExtensions) const
	{
		Super::GetAssetActionButtonExtensions(InAssetData, OutExtensions);
	}
} // namespace UE::Subsonic
#undef LOCTEXT_NAMESPACE // SubsonicEditor
