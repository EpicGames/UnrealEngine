// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"

#include "SubsonicEventCollectionObjects.h"

#include "SubsonicAssetDefinitions.generated.h"

namespace UE::Subsonic
{
	UCLASS()
	class UAssetDefinition_SubsonicEventCollection : public UAssetDefinitionDefault_AudioDiffable
	{
		GENERATED_BODY()

	public:
		// UAssetDefinition Begin
		virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SubsonicEventCollection", "Subsonic Event Collection"); }
		virtual FLinearColor GetAssetColor() const override;
		virtual TSoftClassPtr<UObject> GetAssetClass() const override;
		virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
		virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;

		virtual bool GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const override;
		virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
		virtual void GetAssetActionButtonExtensions(const FAssetData& InAssetData, TArray<FAssetButtonActionExtension>& OutExtensions) const override;

 		virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
 		virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
		// UAssetDefinition End
	};
} // namespace UE::Subsonic