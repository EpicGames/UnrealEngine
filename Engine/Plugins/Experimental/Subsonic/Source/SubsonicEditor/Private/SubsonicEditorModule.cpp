// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SubsonicEventCollectionViews.h"
#include "SubsonicSlateStyle.h"


#define LOCTEXT_NAMESPACE "SubsonicEditor"

namespace UE::Subsonic::Editor
{
	class FModule : public IModuleInterface
	{
		struct ICollectionViewEntryBase { virtual ~ICollectionViewEntryBase() = default; };
		using FViewEntryPtr = TUniquePtr<ICollectionViewEntryBase>;

		template <typename TViewClass, typename TCustomizationClass>
		class FCollectionViewEntry : public ICollectionViewEntryBase
		{
			FName ClassName;

			FCollectionViewEntry(FPropertyEditorModule& PropertyModule)
			{
				UClass* ViewClass = TViewClass::StaticClass();
				check(ViewClass);
				ClassName = ViewClass->GetFName();
				PropertyModule.RegisterCustomClassLayout(
					ClassName,
					FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<TCustomizationClass>(); }));
			}

		public:
			~FCollectionViewEntry()
			{
				if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
				{
					PropertyModule->UnregisterCustomClassLayout(ClassName);
				}
			}

			static FViewEntryPtr Register(FPropertyEditorModule& PropertyModule)
			{
				return FViewEntryPtr(new FCollectionViewEntry<TViewClass, TCustomizationClass>(PropertyModule));
			}
		};

		TArray<FViewEntryPtr> CollectionViewEntries;

	public:
		virtual void StartupModule() override
		{
			// Register Custom Collection Class Views
			{
				FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
				CollectionViewEntries.Add(FCollectionViewEntry<USubsonicEventTreeDetailsView, FEventTreeSelectionDetailCustomization>::Register(PropertyModule));
				CollectionViewEntries.Add(FCollectionViewEntry<USubsonicCollectionParametersView, FCollectionParametersDetailCustomization>::Register(PropertyModule));
			}

			StyleSet = MakeShared<FSlateStyle>();
		}

		virtual void ShutdownModule() override
		{
			CollectionViewEntries.Reset();
			StyleSet.Reset();
		}

		TSharedPtr<FSlateStyleSet> StyleSet;
	};
} // namespace UE::Subsonic::Editor

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::Subsonic::Editor::FModule, SubsonicEditor);
