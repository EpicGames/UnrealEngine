// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicEditorSubsystem.h"

#include "SubsonicEventCollection.h"
#include "UObject/UObjectIterator.h"


namespace UE::Subsonic
{
	void USubsonicEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
	{
		Super::Initialize(Collection);
		RebuildActionStructChildCache();
	}

	void USubsonicEditorSubsystem::Deinitialize()
	{
		Super::Deinitialize();
	}

	void USubsonicEditorSubsystem::RebuildActionStructChildCache()
	{
		using namespace UE::Subsonic::Core;

		ActionStructs.Empty();
		for (TObjectIterator<UScriptStruct> It; It; ++It)
		{
			if (UScriptStruct* Struct = *It)
			{
				if (Struct->IsChildOf(FSubsonicEventActionBase::StaticStruct()))
				{
					ActionStructs.Add(TWeakObjectPtr<UScriptStruct>(Struct));
				}
			}
		}
	}

	void USubsonicEditorSubsystem::ForEachActionStruct(TFunctionRef<void(const UScriptStruct& Struct)> StructIter) const
	{
		for (const TWeakObjectPtr<const UScriptStruct>& Class : ActionStructs)
		{
			if (const UScriptStruct* Struct = Class.Get())
			{
				StructIter(*Struct);
			}
		}
	}
} // namespace UE::Subsonic
