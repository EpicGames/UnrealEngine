// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "EditorSubsystem.h"
#include "Templates/Function.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "SubsonicEditorSubsystem.generated.h"


namespace UE::Subsonic
{
	/**
	 * Interface to the Subsonic Editor Functionality.
	 */
	UCLASS(MinimalAPI, BlueprintType)
	class USubsonicEditorSubsystem final : public UEditorSubsystem
	{
		GENERATED_BODY()

	public:
		void Initialize(FSubsystemCollectionBase& Collection) override;
		void Deinitialize() override;

		// Rebuilds the action struct child cache (called on initialize,
		// but available to refresh if new modules are loaded or unloaded).
		// Cached to allow for faster iteration over projects with large numbers
		// of registered struct types.
		void RebuildActionStructChildCache();

		// Iterates over cached action struct child pointers.
		void ForEachActionStruct(TFunctionRef<void(const UScriptStruct& Struct)> StructIter) const;

	private:
		TArray<TWeakObjectPtr<const UScriptStruct>> ActionStructs;
	};
} // namespace UE::Subsonic
