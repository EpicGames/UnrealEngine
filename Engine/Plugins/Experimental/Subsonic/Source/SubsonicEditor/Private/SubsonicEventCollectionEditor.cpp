// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicEventCollectionEditor.h"

#include "AudioDevice.h"
#include "AudioDeviceHandle.h"
#include "AudioDeviceManager.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameplayTagContainer.h"
#include "ISinglePropertyView.h"
#include "Input/DragAndDrop.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SGameplayTagCombo.h"
#include "ScopedTransaction.h"
#include "StandardActions/SubsonicAction_GeneratorSource.h"
#include "StandardEventSubscribers/SubsonicGeneratorSourceSubscriber.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleRegistry.h"
#include "SubsonicBindingUtils.h"
#include "SubsonicDeviceUtils.h"
#include "SubsonicEditorSubsystem.h"
#include "SubsonicEventCollection.h"
#include "SubsonicEventCollectionObjects.h"
#include "SubsonicEventCollectionEditorCommands.h"
#include "SubsonicExecutor.h"
#include "SubsonicPropertyBindingExtension.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SubsonicEditor"


namespace UE::Subsonic::Editor
{
	void TransactEventCollection(FText Description, USubsonicEventCollection& Collection, TFunctionRef<void(Core::FSubsonicEventCollectionDefinition&)> TransactionFunc)
	{
		FScopedTransaction Transaction(Description);
		Collection.Modify();
		TransactionFunc(Collection.GetMutableDefinition());
	}

	namespace EventCollectionEditorPrivate
	{
		const FName TabName_EventTree = "SubsonicEventCollectionEditor_EventTree";
		const FName TabName_Inspector = "SubsonicEventCollectionEditor_Inspector";
		const FName TabName_Parameters = "SubsonicEventCollectionEditor_Parameters";

		constexpr double IconLerpDuration = 0.5;

		class FActionDragDropOp : public FDragDropOperation
		{
		public:
			DRAG_DROP_OPERATOR_TYPE(FActionDragDropOp, FDragDropOperation)

			Core::FActionHandle SourceAction;

			static TSharedRef<FActionDragDropOp> New(Core::FActionHandle InAction)
			{
				TSharedRef<FActionDragDropOp> Op = MakeShared<FActionDragDropOp>();
				Op->SourceAction = MoveTemp(InAction);
				Op->MouseCursor = EMouseCursor::GrabHandClosed;
				Op->Construct();
				return Op;
			}
		};

		Audio::FDeviceId GetPreviewDeviceId()
		{
			if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
			{
				if (FAudioDeviceHandle ActiveDevice = DeviceManager->GetActiveAudioDevice(); ActiveDevice.IsValid())
				{
					return ActiveDevice->DeviceID;
				}
			}

			return INDEX_NONE;
		}

		const FSlateBrush& GetSlateBrushSafe(FName InName)
		{
			const ISlateStyle* MetaSoundStyle = FSlateStyleRegistry::FindSlateStyle("SubsonicStyle");
			if (ensureMsgf(MetaSoundStyle, TEXT("Missing slate style 'SubsonicStyle'")))
			{
				const FSlateBrush* Brush = MetaSoundStyle->GetBrush(InName);
				if (ensureMsgf(Brush, TEXT("Missing brush '%s'"), *InName.ToString()))
				{
					return *Brush;
				}
			}

			if (const FSlateBrush* NoBrush = FAppStyle::GetBrush("NoBrush"))
			{
				return *NoBrush;
			}

			static const FSlateBrush NullBrush;
			return NullBrush;
		}

		TSharedPtr<IPropertyHandle> GetEventTagPropertyHandle(TSharedRef<ISinglePropertyView> CollectionView, FName EventName)
		{
			TSharedPtr<IPropertyHandle> CollectionProperty = CollectionView->GetPropertyHandle();
			if (!CollectionProperty)
			{
				return { };
			}

			TSharedPtr<IPropertyHandle> EventsHandle = CollectionProperty->GetChildHandle(Core::FSubsonicEventCollectionDefinition::GetEventsPropertyName());
			if (!EventsHandle)
			{
				return { };
			}

			TSharedPtr<IPropertyHandleMap> EventMapHandle = EventsHandle->AsMap();
			if (!EventMapHandle)
			{
				return { };
			}

			uint32 NumEvents = 0;
			if (EventMapHandle->GetNumElements(NumEvents) != FPropertyAccess::Success || NumEvents <= 0)
			{
				return { };
			}

			for (uint32 EventIndex = 0; EventIndex < NumEvents; ++EventIndex)
			{
				TSharedRef<IPropertyHandle> EventEntryHandle = EventMapHandle->GetElement(EventIndex);
				if (TSharedPtr<IPropertyHandle> EventKeyHandle = EventEntryHandle->GetKeyHandle(); EventKeyHandle.IsValid())
				{
					FName Name;
					TSharedPtr<IPropertyHandle> TagNameHandle = EventKeyHandle->GetChildHandle("TagName");
					if (TagNameHandle.IsValid()
						&& TagNameHandle->GetValue(Name) == FPropertyAccess::Success
						&& EventName == Name)
					{
						return EventKeyHandle;
					}
				}
			}

			return { };
		}
	} // namespace EventCollectionEditorPrivate

	const FName FEventCollectionEditor::EditorName = "SubsonicEventCollectionEditor";

	TSharedRef<FEventCollectionEditor::SHandleVariantTableRow> FEventCollectionEditor::InitDraggableEventRow(TSharedPtr<FEventCollectionEditor::FHandleVariant> HandleVariant, const TSharedRef<STableViewBase>& TableView)
	{
		using namespace EventCollectionEditorPrivate;

		if (!HandleVariant.IsValid())
		{
			return SNew(SHandleVariantTableRow, TableView);
		}

		return SNew(SHandleVariantTableRow, TableView)
		.OnDragDetected_Lambda([this, HandleVariant](const FGeometry& Geometry, const FPointerEvent& MouseEvent)
		{
			using namespace UE::Subsonic::Core;

			if (const USubsonicEventCollection* Collection = GetEventCollection())
			{
				if (HandleVariant->IsAction())
				{
					FActionHandle ActionHandle = HandleVariant->GetAction();
					if (const FSubsonicEventActionDefinition* Action = Collection->GetDefinition().FindAction(ActionHandle))
					{
						return FReply::Handled().BeginDragDrop(FActionDragDropOp::New(MoveTemp(ActionHandle)));
					}
				}
			}
			return FReply::Unhandled();
		})
		.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone InitZone, TSharedPtr<FHandleVariant> TargetVariant)
		{
			using namespace UE::Subsonic::Core;

			TOptional<EItemDropZone> ToReturn;
			if (TargetVariant.IsValid())
			{
				TSharedPtr<FDragDropOperation> Op = DragDropEvent.GetOperation();
				if (Op.IsValid() && Op->IsOfType<FActionDragDropOp>())
				{
					// If event, add to end of event
					if (TargetVariant->IsEvent())
					{
						ToReturn = EItemDropZone::OntoItem;
					}
					else
					{
						// Reject only a self-drop (same parent event, same index).  Cross-event drops are allowed.
						const FActionHandle& SourceAction = StaticCastSharedRef<FActionDragDropOp>(Op.ToSharedRef())->SourceAction;
						if (SourceAction != TargetVariant->GetAction())
						{
							ToReturn = InitZone;
						}
					}
				}
			}
			return ToReturn;
		})
		.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FHandleVariant> HandleVariant)
		{
			TSharedPtr<FDragDropOperation> Op = DragDropEvent.GetOperation();
			if (Op.IsValid() && Op->IsOfType<FActionDragDropOp>())
			{
				if (HandleVariant.IsValid())
				{
					const FActionHandle& SourceAction = StaticCastSharedRef<FActionDragDropOp>(Op.ToSharedRef())->SourceAction;
					MoveDroppedAction(SourceAction, *HandleVariant.Get(), DropZone);
				}
			}
			return FReply::Handled();
		});
	}

	TSharedRef<ITableRow> FEventCollectionEditor::OnEventTree_GenerateRow(TSharedPtr<FEventCollectionEditor::FHandleVariant> HandleVariant, const TSharedRef<STableViewBase>& TableView)
	{
		using namespace UE::Subsonic::Core;

		TSharedRef<SHandleVariantTableRow> NewRow = InitDraggableEventRow(HandleVariant, TableView);

		const USubsonicEventCollection* Collection = GetEventCollection();
		if (!HandleVariant.IsValid() || !Collection)
		{
			return NewRow;
		}

		FText RowDisplayName = FText::FromName(NAME_None); // Valid display name for actions with no struct type set
		constexpr float RowIndent = 5.f;
		if (HandleVariant->IsAction())
		{
			if (const FSubsonicEventActionDefinition* Action = Collection->GetDefinition().FindAction(HandleVariant->GetAction()))
			{
				if (const UScriptStruct* ActionStruct = Action->GetAction().GetScriptStruct())
				{
					RowDisplayName = ActionStruct->GetDisplayNameText();
				}
			}
		}
		else if (HandleVariant->IsEvent())
		{
			RowDisplayName = FText::FromName(HandleVariant->GetEvent().EventName);
		}
		else
		{
			checkNoEntry();
		}

		TSharedPtr<SWidget> NameWidget = SNullWidget::NullWidget;
		if (HandleVariant->IsEvent())
		{
			if (CollectionView.IsValid())
			{
				TSharedPtr<IPropertyHandle> EventNameProperty = EventCollectionEditorPrivate::GetEventTagPropertyHandle(CollectionView.ToSharedRef(), HandleVariant->GetEvent().EventName);
				if (EventNameProperty.IsValid())
				{
					NameWidget = SNew(SGameplayTagCombo)
					.Filter(TAG_SubsonicCore.GetTag().GetTagName().ToString())
					.PropertyHandle(EventNameProperty)
					.OnTagChanged_Lambda([this, HandleRef = HandleVariant.ToSharedRef()](const FGameplayTag NewTag)
					{
						if (NewTag.GetTagName() != HandleRef->GetEvent().EventName)
						{
							FEventHandle NewEvent = HandleRef->GetEvent();
							NewEvent.EventName = NewTag.GetTagName();
							HandleRef->Set(MoveTemp(NewEvent));
							RebuildEventTree(true /* bRebuildListView */);
						}
					});
				}
			}
		}
		else if (const FSubsonicEventActionDefinition* Action = Collection->GetDefinition().FindAction(HandleVariant->GetAction()))
		{
			FText ActionDisplayName = FText::FromName("None");
			if (const UScriptStruct* ActionStruct = Action->GetAction().GetScriptStruct())
			{
				ActionDisplayName = ActionStruct->GetDisplayNameText();
			}

			TSharedPtr<SSearchableComboBox> ComboBox = SNew(SSearchableComboBox)
			.OptionsSource(&ActionComboValues)
			.OnGenerateWidget_Lambda([this](TSharedPtr<FString> InItem) -> TSharedRef<SWidget>
			{
				TSharedRef<SWidget> EntryWidget = SNullWidget::NullWidget;
				if (InItem.IsValid())
				{
					const FString& ItemStr = *InItem.Get();
					if (USubsonicEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<USubsonicEditorSubsystem>())
					{
						EditorSubsystem->ForEachActionStruct([this, &EntryWidget, &ItemStr](const UScriptStruct& ActionStruct)
						{
							if (ActionStruct.GetFName() == ItemStr)
							{
								EntryWidget = SNew(STextBlock).Text(ActionStruct.GetDisplayNameText());
							}
						});
					}
				}

				return EntryWidget;
			})
			.OnSelectionChanged_Lambda([this, HandleRef = HandleVariant.ToSharedRef()](TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
			{
				if (SelectInfo != ESelectInfo::OnNavigation && NewSelection.IsValid())
				{
					TransactEventCollection(FText::Format(LOCTEXT("SetActionDescription", "Set action type to '{0}'"), FText::FromString(*NewSelection)), [&](FSubsonicEventCollectionDefinition& Definition)
					{
						if (FSubsonicEventActionDefinition* Action = Definition.FindMutableAction(HandleRef->GetAction()))
						{
							const FName NewSelectionName = FName(*NewSelection);
							if (USubsonicEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<USubsonicEditorSubsystem>())
							{
								EditorSubsystem->ForEachActionStruct([this, &Action, NewSelectionName, &HandleRef](const UScriptStruct& ActionStruct)
								{
									if (ActionStruct.GetFName() == NewSelectionName)
									{
										TInstancedStruct<FSubsonicEventActionBase> NewConfig;
										NewConfig.InitializeAsScriptStruct(&ActionStruct);
										FSubsonicEventActionBase::InitializeDefaultActionName(NewConfig, HandleRef->GetEvent());
										Action->SetAction(MoveTemp(NewConfig));
										RebuildEventTree(true /* bRebuildListView */);
									}
								});
							}
						}
					});
				}
			})
			.Content()
			[
				SNew(STextBlock).Text(ActionDisplayName)
			];

			SAssignNew(NameWidget, SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
				.AutoWidth()
			.Padding(0.f, 0.f, 5.0f, 0.f)
			[
				ComboBox.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.f, 0.f, 5.0f, 0.f)
			[
				SNew(STextBlock)
				.Font(FCoreStyle::GetDefaultFontStyle("Italic", 10))
				.Text_Lambda([this, HandleRef = HandleVariant.ToSharedRef()]()
				{
					if (const USubsonicEventCollection* Collection = GetEventCollection())
					{
						if (const FSubsonicEventActionDefinition* Action = Collection->GetDefinition().FindAction(HandleRef->GetAction()))
						{
							if (const FSubsonicEventActionBase* ActionBase = Action->GetAction().GetPtr())
							{
								return ActionBase->GetDisplayInfo();
							}
						}
					}
					return FText();
				})
			];
		}

		TSharedPtr<SHorizontalBox> RowBox;
		TSharedRef<SHorizontalBox> NameBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			.Padding(0.f, 0.f, 5.0f, 0.f)
		[
			NameWidget.ToSharedRef()
		];

		auto IsHovered = [RowPtr = NewRow.ToWeakPtr()]()
		{
			if (RowPtr.IsValid() && RowPtr.Pin()->IsHovered())
			{
				return true;
			}
			return false;
		};
		TSharedRef<FHandleVariant> HandleRef = HandleVariant.ToSharedRef();

		if (HandleVariant->IsEvent())
		{
			NameBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this, HandleRef]()
				{
					TransactEvent(HandleRef->GetEvent(), LOCTEXT("ToggleEventAccessor", "Toggle event {0} accessor"), [](FSubsonicEvent& Event)
					{
						const bool bWasPublic = Event.GetIsPublic();
						Event.SetIsPublic(!bWasPublic);
					});
					return FReply::Handled();
				})
				.ToolTipText_Lambda([this, HandleRef]()
				{
					if (const USubsonicEventCollection* Collection = GetEventCollection())
					{
						if (const FSubsonicEvent* Event = Collection->GetDefinition().FindEvent(HandleRef->GetEvent()))
						{
							if (Event->GetIsPublic())
							{
								return LOCTEXT("IsEventPublic_Tooltip", "Event can be executed from outside of internal actions.");
							}
						}
					}
					return LOCTEXT("IsEventPrivate_Tooltip", "Event can only be executed from actions contained in this event collection (not from external scripts or other collections).");
				})
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(0)
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image_Lambda([this, HandleRef]()
					{
						if (const USubsonicEventCollection* Collection = GetEventCollection())
						{
							if (const FSubsonicEvent* Event = Collection->GetDefinition().FindEvent(HandleRef->GetEvent()); Event && Event->GetIsPublic())
							{
								return FAppStyle::GetBrush("Kismet.VariableList.ExposeForInstance");
							}
						}

						return FAppStyle::GetBrush("Kismet.VariableList.HideForInstance");
					})
					.ColorAndOpacity(FSlateColor::UseForeground())
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				]
			];
			NameBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this, HandleRef]()
				{
					TransactEvent(HandleRef->GetEvent(), LOCTEXT("ToggleAutoAudition", "Toggle event {0} auto audition"), [](FSubsonicEvent& Event)
					{
						Event.SetAutoAudition(!Event.GetAutoAudition());
					});
					return FReply::Handled();
				})
				.ToolTipText_Lambda([this, HandleRef]()
				{
					if (const USubsonicEventCollection* Collection = GetEventCollection())
					{
						if (const FSubsonicEvent* Event = Collection->GetDefinition().FindEvent(HandleRef->GetEvent()))
						{
							if (Event->GetAutoAudition())
							{
								return LOCTEXT("AutoAuditionEnabled_Tooltip", "Auto Audition is enabled. Event will execute automatically when audition starts.");
							}
						}
					}
					return LOCTEXT("AutoAuditionDisabled_Tooltip", "Auto Audition is disabled. Click to enable.");
				})
				.Visibility_Lambda([this, HandleRef, IsHovered]()
				{
					if (IsHovered())
					{
						return EVisibility::Visible;
					}

					if (const USubsonicEventCollection* Collection = GetEventCollection())
					{
						if (const FSubsonicEvent* Event = Collection->GetDefinition().FindEvent(HandleRef->GetEvent()))
						{
							if (Event->GetAutoAudition())
							{
								return EVisibility::Visible;
							}
						}
					}
					return EVisibility::Hidden;
				})
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(0)
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(&EventCollectionEditorPrivate::GetSlateBrushSafe("SubsonicEditor.Arm.Small"))
					.ColorAndOpacity_Lambda([this, HandleRef]() -> FSlateColor
					{
						if (const USubsonicEventCollection* Collection = GetEventCollection())
						{
							if (const FSubsonicEvent* Event = Collection->GetDefinition().FindEvent(HandleRef->GetEvent()))
							{
								if (Event->GetAutoAudition())
								{
									return FSlateColor(FLinearColor::Red);
								}
							}
						}
						return FSlateColor::UseForeground();
					})
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				]
			];
			NameBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this, HandleRef]()
				{
					ExecutePreviewEvent(HandleRef->GetEvent());
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("Test Execute", "Execute the given event using the editor world as a test."))
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(0)
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(&EventCollectionEditorPrivate::GetSlateBrushSafe("SubsonicEditor.Play.Small"))
					.ColorAndOpacity_Lambda([this, EventName = HandleRef->GetEvent().EventName]() -> FSlateColor
					{
						if (const double* LerpTime = EventExecuteLerpTimes.Find(EventName))
						{
							const double Elapsed = FMath::Max(0.0, FSlateApplication::Get().GetCurrentTime() - *LerpTime);
							if (Elapsed < EventCollectionEditorPrivate::IconLerpDuration)
							{
								const float Alpha = static_cast<float>(Elapsed / EventCollectionEditorPrivate::IconLerpDuration);
								const FLinearColor ForegroundColor = FStyleColors::Foreground.GetSpecifiedColor();
								return FSlateColor(FLinearColor::LerpUsingHSV(FLinearColor::Green, ForegroundColor, Alpha));
							}
						}
						return FSlateColor::UseForeground();
					})
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.Visibility_Lambda([this, EventName = HandleRef->GetEvent().EventName, IsHovered]()
					{
						if (IsHovered())
						{
							return EVisibility::Visible;
						}

						if (const double* LerpTime = EventExecuteLerpTimes.Find(EventName))
						{
							const double Elapsed = FMath::Max(0.0, FSlateApplication::Get().GetCurrentTime() - *LerpTime);
							if (Elapsed < EventCollectionEditorPrivate::IconLerpDuration)
							{
								return EVisibility::Visible;
							}
						}
						return EVisibility::Hidden;
					})
				]
			];
		}

		NewRow->SetContent(
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(RowIndent, 5.0f, 5.0f, 5.0f)
			[
				SAssignNew(RowBox, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				[ NameBox ]
			]
		);

		RowBox->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this, HandleRef]()
				{
					InsertAction(*HandleRef);
					RebuildEventTree(true /* bRebuildListView */);
					return FReply::Handled();
				})
				.ToolTipText_Lambda([this, HandleRef]()
				{
					return FText::Format(
						HandleRef->IsEvent()
							? LOCTEXT("AddActionToEndofEventToolTip_Format", "Add action to end of event '{0}' array")
							: LOCTEXT("InsertActionInEventToolTip_Format", "Insert new action in event '{0}' array"),
						FText::FromName(HandleRef->GetEvent().EventName));
				})
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(0)
				.IsFocusable(false)
				.Visibility_Lambda([IsHovered]() { return IsHovered() ? EVisibility::Visible : EVisibility::Hidden; })
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.AddCircle"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this, HandleRef]()
				{
					const FString HandleDesc = HandleRef->IsAction() ? HandleRef->GetAction().ToString() : HandleRef->GetEvent().ToString();
					FText TransactionDesc = FText::Format(LOCTEXT("RemoveHandleDescription", "Remove {0} from Subsonic Event Collection"), FText::FromString(HandleDesc));
					TransactEventCollection(TransactionDesc, [&](FSubsonicEventCollectionDefinition& Definition)
					{
						const FEventHandle& EventHandle = HandleRef->GetEvent();
						bool bAddedHandleItem = HandleRef->IsEvent()
							? Definition.RemoveEvent(EventHandle)
							: Definition.RemoveAction(HandleRef->GetAction());
						if (bAddedHandleItem)
						{
							RebuildEventTree(true /* bRebuildListView */);
						}
					});
					return FReply::Handled();
				})
				.ToolTipText(HandleRef->IsEvent()
					? FText::Format(LOCTEXT("DeleteEventToolTip_Format", "Delete event '{0}' from Subsonic Event Collection"), FText::FromName(HandleRef->GetEvent().EventName))
					: LOCTEXT("DeleteActionToolTip_Format", "Delete action from event"))
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(0)
				.Visibility_Lambda([IsHovered]() { return IsHovered() ? EVisibility::Visible : EVisibility::Hidden; })
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];

		return NewRow;
	}

	void FEventCollectionEditor::OnEventTree_GetChildren(TSharedPtr<FHandleVariant> InParent, TArray<TSharedPtr<FHandleVariant>>& OutChildren)
	{
		if (InParent.IsValid() && InParent->IsEvent())
		{
			if (const TArray<TSharedPtr<FHandleVariant>>* Children = EventTreeChildHandles.Find(InParent->GetEvent().EventName))
			{
				OutChildren = *Children;
			}
		}
	}

	void FEventCollectionEditor::OnEventTree_RowSelected(TSharedPtr<FHandleVariant> HandleVariant, ESelectInfo::Type SelectInfo)
	{
		if (TreeDetailsView.IsValid() && HandleVariant.IsValid())
		{
			if (HandleVariant->IsAction())
			{
				const FActionHandle ActionHandle = HandleVariant->GetAction();
				TreeDetailsView->Event = ActionHandle.Event.EventName;
				TreeDetailsView->ActionIndex = ActionHandle.Index;
			}
			else
			{
				TreeDetailsView->Event = HandleVariant->GetEvent().EventName;
				TreeDetailsView->ActionIndex = INDEX_NONE;
			}
		}

		if (EventTreeInspectorView.IsValid())
		{
			EventTreeInspectorView->ForceRefresh();
		}
	}

	void FEventCollectionEditor::SelectEventInTree(FName EventName)
	{
		if (!EventTreeView.IsValid())
		{
			return;
		}
		for (const TSharedPtr<FHandleVariant>& Handle : EventTreeRootHandles)
		{
			if (Handle.IsValid() && Handle->IsEvent() && Handle->GetEvent().EventName == EventName)
			{
				EventTreeView->SetSelection(Handle, ESelectInfo::Direct);
				return;
			}
		}
	}

	void FEventCollectionEditor::ExpandAllEvents()
	{
		if (EventTreeView.IsValid())
		{
			for (const TSharedPtr<FHandleVariant>& RootHandle : EventTreeRootHandles)
			{
				EventTreeView->SetItemExpansion(RootHandle, true);
			}
		}
	}

	void FEventCollectionEditor::CollapseAllEvents()
	{
		if (EventTreeView.IsValid())
		{
			for (const TSharedPtr<FHandleVariant>& RootHandle : EventTreeRootHandles)
			{
				EventTreeView->SetItemExpansion(RootHandle, false);
			}
		}
	}

	void FEventCollectionEditor::TryInvokeTab(FName TabName) const
	{
		if (TabManager.IsValid())
		{
			TabManager->TryInvokeTab(TabName);
		}
	}

	void FEventCollectionEditor::RebuildEventTree(bool bRebuildListView)
	{
		ActionComboValues.Reset();
		EventTreeRootHandles.Reset();
		EventTreeChildHandles.Reset();

		if (!GEditor)
		{
			return;
		}

		USubsonicEventCollection* EventCollection = GetEventCollection();
		if (!EventCollection)
		{
			return;
		}

		// Remove Lerp entries for events that no longer exist (renamed or deleted)
		if (EventExecuteLerpTimes.Num() > 0)
		{
			const TMap<FGameplayTag, FSubsonicEvent>& Events = EventCollection->GetDefinition().GetEvents();
			for (auto It = EventExecuteLerpTimes.CreateIterator(); It; ++It)
			{
				bool bFound = false;
				for (const auto& EventPair : Events)
				{
					if (EventPair.Key.GetTagName() == It->Key)
					{
						bFound = true;
						break;
					}
				}
				if (!bFound)
				{
					It.RemoveCurrent();
				}
			}
		}

		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		CollectionView = PropertyEditor.CreateSingleProperty(EventCollection, USubsonicEventCollection::GetDefinitionPropertyName(), { });

		if (USubsonicEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<USubsonicEditorSubsystem>())
		{
			EditorSubsystem->ForEachActionStruct([this](const UScriptStruct& ActionStruct)
			{
				if (!ActionStruct.HasMetaData("Hidden"))
				{
					ActionComboValues.Add(MakeShared<FString>(ActionStruct.GetFName().ToString()));
				}
			});
		}

		using FTagEventPair = TPair<FGameplayTag, FSubsonicEvent>;
		using FTagEventPtrPair = TPair<FGameplayTag, const FSubsonicEvent*>;
		TArray<FTagEventPtrPair> SortedEvents;
		Algo::TransformIf(EventCollection->GetDefinition().GetEvents(), SortedEvents,
		[this](const FTagEventPair& Pair)
		{
			if (EventViewFilter.IsEmpty())
			{
				return true;
			}
			if (Pair.Key.GetTagName().ToString().Contains(EventViewFilter))
			{
				return true;
			}
			return Pair.Value.GetActionCollection().ContainsByPredicate([this](const FSubsonicEventActionDefinition& Action)
			{
				const UScriptStruct* Struct = Action.GetAction().GetScriptStruct();
				return Struct && Struct->GetDisplayNameText().ToString().Contains(EventViewFilter);
			});
		},
		[this](const FTagEventPair& Pair)
		{
			return FTagEventPtrPair(Pair.Key, &Pair.Value);
		});

		SortedEvents.Sort([](const FTagEventPtrPair& A, const FTagEventPtrPair& B)
		{
			return A.Key.GetTagName().LexicalLess(B.Key.GetTagName());
		});
		for (const FTagEventPtrPair& Pair : SortedEvents)
		{
			FHandleVariant EventVariant;
			const FEventHandle EventHandle = { .Collection = EventCollection->GetHandle(), .EventName = Pair.Key.GetTagName() };
			EventVariant.Set(EventHandle);
			TSharedPtr<FHandleVariant> EventPtr = MakeShared<FHandleVariant>(MoveTemp(EventVariant));
			EventTreeRootHandles.Add(EventPtr);

			TArray<TSharedPtr<FHandleVariant>>& ChildHandles = EventTreeChildHandles.Add(Pair.Key.GetTagName());
			int32 ActionIndex = 0;
			for (const FSubsonicEventActionDefinition& ActionDefinition : Pair.Value->GetActionCollection())
			{
				FHandleVariant ActionVariant;
				const FActionHandle ActionHandle = { .Event = EventHandle, .Index = ActionIndex++ };
				ActionVariant.Set(ActionHandle);
				ChildHandles.Add(MakeShared<FHandleVariant>(MoveTemp(ActionVariant)));
			}
		}

		if (bRebuildListView && EventTreeView.IsValid())
		{
			EventTreeView->RequestTreeRefresh();

			// Default all events to expanded
			for (const TSharedPtr<FHandleVariant>& RootHandle : EventTreeRootHandles)
			{
				EventTreeView->SetItemExpansion(RootHandle, true);
			}
		}
	}

	void FEventCollectionEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_SubsonicEventCollectionEditor", "Subsonic EventCollection Editor"));

		FDetailsViewArgs DetailViewArgs;
		DetailViewArgs.bHideSelectionTip = true;
		DetailViewArgs.NotifyHook = this;

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		ParametersDetailsView = PropertyModule.CreateDetailView(DetailViewArgs);
		InTabManager->RegisterTabSpawner(EventCollectionEditorPrivate::TabName_Parameters, FOnSpawnTab::CreateLambda([ViewRef = ParametersDetailsView.ToSharedRef()](const FSpawnTabArgs&)
		{
			return SNew(SDockTab).Label(LOCTEXT("SubsonicEventCollectionParametersTitle", "Parameters")) [ ViewRef ];
		}))
		.SetDisplayName(LOCTEXT("ParametersTab", "Parameters"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(EventCollectionEditorPrivate::TabName_EventTree, FOnSpawnTab::CreateLambda([ViewRef = InitEventTreeView()](const FSpawnTabArgs& Args)
		{
			return SNew(SDockTab).Label(LOCTEXT("SubsonicEventCollectionEventsTreeTitle", "Event Tree")) [ ViewRef ];
		}))
		.SetDisplayName(LOCTEXT("EventTreeTab", "Event Tree"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));


		DetailViewArgs.bAllowSearch = false;
		EventTreeInspectorView = PropertyModule.CreateDetailView(DetailViewArgs);
		InTabManager->RegisterTabSpawner(EventCollectionEditorPrivate::TabName_Inspector, FOnSpawnTab::CreateLambda([ViewRef = EventTreeInspectorView.ToSharedRef()](const FSpawnTabArgs&)
		{
			return SNew(SDockTab).Label(LOCTEXT("SubsonicEventCollectionInspectorTitle", "Inspector")) [ ViewRef ];
		}))
		.SetDisplayName(LOCTEXT("InspectorTab", "Inspector"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Properties"));
		TWeakPtr<FEventCollectionEditor> WeakThis(SharedThis(this));
		EventTreeInspectorView->SetExtensionHandler(MakeShared<FSubsonicPropertyBindingExtension>(
			[WeakThis](FName EventName)
			{
				if (TSharedPtr<FEventCollectionEditor> Editor = WeakThis.Pin())
				{
					// None = collection level parameters, so just invoke/bring focus to that tab
					if (EventName.IsNone())
					{
						Editor->GetTabManager()->TryInvokeTab(EventCollectionEditorPrivate::TabName_Parameters);
					}
					else
					{
						Editor->SelectEventInTree(EventName);
					}
				}
			}
		));

		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	}

	void FEventCollectionEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

		InTabManager->UnregisterTabSpawner(EventCollectionEditorPrivate::TabName_EventTree);
		InTabManager->UnregisterTabSpawner(EventCollectionEditorPrivate::TabName_Inspector);
		InTabManager->UnregisterTabSpawner(EventCollectionEditorPrivate::TabName_Parameters);
	}

	FEventCollectionEditor::~FEventCollectionEditor()
	{
		if (USubsonicEventCollection* Collection = GetEventCollection())
		{
			Collection->GetMutableDefinition().GetOnStaleBindingsDetectedDelegate().Remove(StaleBindingsDelegateHandle);
		}

		StopAudition();

		if (GEditor)
		{
			GEditor->UnregisterForUndo(this);
		}
	}

	void FEventCollectionEditor::InitBindings()
	{
		using namespace UE::Subsonic::Core;
		using namespace UE::Subsonic::Core::BindingUtils;

		USubsonicEventCollection* Collection = GetEventCollection();
		if (!Collection)
		{
			return;
		}

		FSubsonicEventCollectionDefinition& Definition = Collection->GetMutableDefinition();
		Definition.RemoveStaleBindings(Collection->GetHandle());
		BindingUtils::FOnStaleBindingsDetected& StaleBindingsDelegate = Definition.GetOnStaleBindingsDetectedDelegate();
		StaleBindingsDelegate.Remove(StaleBindingsDelegateHandle);

		// Register handler to show a confirmation dialog when a structural parameter change
		// (type or name change) would invalidate editor property bindings.
		StaleBindingsDelegateHandle = StaleBindingsDelegate.AddLambda([WeakCol = TWeakObjectPtr<USubsonicEventCollection>(Collection)](const FStaleBindingsMap& StaleBindings, EStaleBindingResponse& OutResponse)
		{
			// Ignore if another delegate handler responded that it reverted on stale binding detection.
			if (OutResponse != EStaleBindingResponse::Revert)
			{
				if (const USubsonicEventCollection* PinnedCol = WeakCol.Get())
				{
					BindingUtils::PromptForStaleBindings(PinnedCol->GetDefinition(), StaleBindings, OutResponse);
				}

				// Deferred to next tick: the current transaction must finish posting before
				// it can be rolled back. The lambda is intentionally fire-and-forget with
				// no captures from the surrounding stack, so no lifetime concerns arise.
				// bCanRedo is false because the reverted parameter change left stale bindings
				// that were declined — re-applying it would reintroduce the same stale state.
				if (OutResponse == EStaleBindingResponse::Revert)
				{
					FTSTicker::GetCoreTicker().AddTicker(TEXT("SubsonicEventCollectionEditor::UndoParameterTransaction"), 0.0f, [](float)
					{
						if (GEditor)
						{
							GEditor->UndoTransaction(false /* bCanRedo */);
						}
						return false;
					});
				}
			}
		});
	}

	void FEventCollectionEditor::Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USubsonicEventCollection& InEventCollection)
	{
		// Support undo/redo
		InEventCollection.SetFlags(RF_Transactional);

		if (GEditor)
		{
			GEditor->RegisterForUndo(this);
		}

		ParametersStack = FTabManager::NewStack()
			->SetSizeCoefficient(0.33f)
			->SetHideTabWell(true)
			->AddTab(EventCollectionEditorPrivate::TabName_Parameters, ETabState::OpenedTab);

		EventTreeStack = FTabManager::NewStack()
			->SetSizeCoefficient(0.33f)
			->SetHideTabWell(true)
			->AddTab(EventCollectionEditorPrivate::TabName_EventTree, ETabState::OpenedTab);

		DetailsStack = FTabManager::NewStack()
			->SetSizeCoefficient(0.34f)
			->SetHideTabWell(true)
			->AddTab(EventCollectionEditorPrivate::TabName_Inspector, ETabState::OpenedTab);

		const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_SubsonicEventCollectionEditor_Layout_v7")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					ParametersStack.ToSharedRef()
				)
				->Split
				(
					EventTreeStack.ToSharedRef()
				)
				->Split
				(
					DetailsStack.ToSharedRef()
				)
			);

		FEventCollectionEditorCommands::Register();
		BuildTransport();

		{
			constexpr bool bCreateDefaultStandaloneMenu = true;
			constexpr bool bCreateDefaultToolbar = true;
			constexpr bool bToolbarFocusable = false;
			constexpr bool bUseSmallToolbarIcons = true;

			FAssetEditorToolkit::InitAssetEditor(
				Mode,
				InitToolkitHost,
				FName("SubsonicEventCollectionEditorApp"),
				StandaloneDefaultLayout,
				bCreateDefaultStandaloneMenu,
				bCreateDefaultToolbar,
				&InEventCollection,
				bToolbarFocusable,
				bUseSmallToolbarIcons);
		}

		USubsonicEventCollection* EventCollection = GetEventCollection();
		check(EventCollection);

		InitBindings();

		RebuildEventTree(true /* bRebuildListView */);

		if (ParametersDetailsView.IsValid())
		{
			ParametersView.Reset(NewObject<USubsonicCollectionParametersView>());
			ParametersView->SetCollection(EventCollection);
			ParametersDetailsView->SetObject(ParametersView.Get());
		}

		if (EventTreeInspectorView.IsValid())
		{
			TreeDetailsView.Reset(NewObject<USubsonicEventTreeDetailsView>());
			TreeDetailsView->SetCollection(EventCollection);
			EventTreeInspectorView->SetObject(TreeDetailsView.Get());
		}

		RegenerateMenusAndToolbars();
		const FEventCollectionEditorCommands& Commands = FEventCollectionEditorCommands::Get();
		ToolkitCommands->MapAction(
			Commands.StartAudition,
			FExecuteAction::CreateSP(this, &FEventCollectionEditor::StartAudition));

		ToolkitCommands->MapAction(
			Commands.StopAudition,
			FExecuteAction::CreateSP(this, &FEventCollectionEditor::StopAudition));

		ToolkitCommands->MapAction(
			Commands.ToggleAudition,
			FExecuteAction::CreateSP(this, &FEventCollectionEditor::ToggleAudition));
	}

	TSharedRef<SWidget> FEventCollectionEditor::InitEventTreeView()
	{
		TSharedRef<SHeaderRow> EventTreeHeader = SNew(SHeaderRow);
		EventTreeHeader->AddColumn(
			SHeaderRow::FColumn::FArguments()
			.ColumnId("Events")
			.DefaultTooltip(LOCTEXT("EventsColumn_Tooltip", "Events that can be executed at runtime by code or script."))
			.VAlignHeader(VAlign_Center)
			.VAlignCell(VAlign_Center)
			.HAlignHeader(HAlign_Left)
			.HAlignCell(HAlign_Fill)
			.HeaderContentPadding(FMargin(10.0f, 5.0f))
			.HeaderContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.Text(LOCTEXT("EventsColumn_DisplayName", "Events"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked_Lambda([this]()
					{
						TransactEventCollection(LOCTEXT("AddEventDescription", "Add new Subsonic event to collection"), [&](Core::FSubsonicEventCollectionDefinition& Definition)
						{
							if (FSubsonicEvent* NewEvent = Definition.AddEvent())
							{
								RebuildEventTree(true /* bRebuildListView */);
							}
						});
						return FReply::Handled();
					})
					.ToolTipText(LOCTEXT("AddEventToolTip", "Add event to collection"))
					.ForegroundColor(FSlateColor::UseForeground())
					.ContentPadding(0)
					.IsFocusable(false)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.AddCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked_Lambda([this]()
					{
						TransactEventCollection(LOCTEXT("ClearAllEventsDescription", "Clear all Subsonic Collection Events"), [&](Core::FSubsonicEventCollectionDefinition& Definition)
						{
							Definition.ClearEvents();
							RebuildEventTree(true /* bRebuildListView */);
						});
						return FReply::Handled();
					})
					.ToolTipText(LOCTEXT("DeleteAllEventsToolTip", "Delete all events in collection"))
					.ForegroundColor(FSlateColor::UseForeground())
					.ContentPadding(0)
					.IsFocusable(false)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
					.HasDownArrow(false)
					.ContentPadding(0)
					.ForegroundColor(FSlateColor::UseForeground())
					.ToolTipText(LOCTEXT("EventTreeOptionsToolTip", "Event tree options"))
					.OnGetMenuContent_Lambda([this]()
					{
						FMenuBuilder MenuBuilder(true, nullptr);
						MenuBuilder.AddMenuEntry(
							LOCTEXT("ExpandAllEvents", "Expand All"),
							LOCTEXT("ExpandAllEvents_Tooltip", "Expand all events in the tree"),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateSP(this, &FEventCollectionEditor::ExpandAllEvents))
						);
						MenuBuilder.AddMenuEntry(
							LOCTEXT("CollapseAllEvents", "Collapse All"),
							LOCTEXT("CollapseAllEvents_Tooltip", "Collapse all events in the tree"),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateSP(this, &FEventCollectionEditor::CollapseAllEvents))
						);
						return MenuBuilder.MakeWidget();
					})
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		);

		auto SetFilterString = [this](const FText& InFilterText)
		{
			EventViewFilter = InFilterText.ToString();
			RebuildEventTree(true /* bRebuildListView */);
		};
		return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(6.f)
			.FillWidth(1.0f)
			[
				SNew(SSearchBox)
				.HintText(LOCTEXT("SearchDetailsHint", "Search"))
				.OnTextChanged_Lambda(SetFilterString)
				.AddMetaData<FTagMetaData>(TEXT("Details.Search"))
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(EventTreeView, STreeView<TSharedPtr<FHandleVariant>>)
			.HeaderRow(EventTreeHeader)
			.TreeItemsSource(&EventTreeRootHandles)
			.OnGetChildren(this, &FEventCollectionEditor::OnEventTree_GetChildren)
			.OnGenerateRow(this, &FEventCollectionEditor::OnEventTree_GenerateRow)
			.OnSelectionChanged(this, &FEventCollectionEditor::OnEventTree_RowSelected)
			.SelectionMode(ESelectionMode::Single)
		];
	}

	void FEventCollectionEditor::TransactEvent(const FEventHandle EventHandle, FText Description, TFunctionRef<void(Core::FSubsonicEvent&)> TransactionFunc) const
	{
		using namespace UE::Subsonic::Core;

		if (USubsonicEventCollection* Collection = GetEventCollection())
		{
			if (FSubsonicEvent* Event = Collection->GetMutableDefinition().FindMutableEvent(EventHandle))
			{
				FScopedTransaction Transaction(Description);
				Collection->Modify();
				TransactionFunc(*Event);
			}
		}
	}

	void FEventCollectionEditor::TransactEventCollection(FText Description, TFunctionRef<void(FSubsonicEventCollectionDefinition&)> TransactionFunc) const
	{
		if (USubsonicEventCollection* Collection = GetEventCollection())
		{
			UE::Subsonic::Editor::TransactEventCollection(Description, *Collection, TransactionFunc);
		}
	}

	USubsonicEventCollection* FEventCollectionEditor::GetEventCollection() const
	{
		return Cast<USubsonicEventCollection>(GetEditingObject());
	}

	FName FEventCollectionEditor::GetToolkitFName() const
	{
		return FEventCollectionEditor::EditorName;
	}

	FText FEventCollectionEditor::GetBaseToolkitName() const
	{
		return LOCTEXT("AppLabel", "Subsonic Event Collection Editor");
	}

	FString FEventCollectionEditor::GetWorldCentricTabPrefix() const
	{
		return LOCTEXT("WorldCentricTabPrefix", "Subsonic Event Collection ").ToString();
	}

	FLinearColor FEventCollectionEditor::GetWorldCentricTabColorScale() const
	{
		if (const ISlateStyle* Style = FSlateStyleRegistry::FindSlateStyle("SubsonicStyle"))
		{
			return Style->GetColor("SubsonicEventCollection.Color");
		}

		return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
	}

	const FSlateBrush* FEventCollectionEditor::GetDefaultTabIcon() const
	{
		if (const ISlateStyle* Style = FSlateStyleRegistry::FindSlateStyle("SubsonicStyle"))
		{
			if (const FSlateBrush* Brush = Style->GetBrush("SubsonicEditor.SubsonicEventCollection.Icon"))
			{
				return Brush;
			}
		}

		if (const FSlateBrush* NoBrush = FAppStyle::GetBrush("NoBrush"))
		{
			return NoBrush;
		}

		static const FSlateBrush NullBrush;
		return &NullBrush;
	}

	FLinearColor FEventCollectionEditor::GetDefaultTabColor() const
	{
		if (const ISlateStyle* Style = FSlateStyleRegistry::FindSlateStyle("SubsonicStyle"))
		{
			return Style->GetColor("SubsonicEventCollection.Color");
		}

		return FAssetEditorToolkit::GetDefaultTabColor();
	}

	Core::FSubsonicEventActionDefinition* FEventCollectionEditor::InsertAction(const FEventCollectionEditor::FHandleVariant& InHandleVariant)
	{
		FSubsonicEventActionDefinition* NewAction= nullptr;

		const FText TransactionDesc = FText::Format(
			InHandleVariant.IsEvent()
			? LOCTEXT("AddActionToEndofEventToolTip_Format", "Add action to end of event '{0}' array")
			: LOCTEXT("InsertActionInEventToolTip_Format", "Insert new action in event '{0}' array"),
			FText::FromName(InHandleVariant.GetEvent().EventName));

		TransactEventCollection(TransactionDesc, [&](FSubsonicEventCollectionDefinition& Definition)
		{
			const FEventHandle& EventHandle = InHandleVariant.GetEvent();
			int32 InsertIndex = InHandleVariant.IsEvent() ? INDEX_NONE : InHandleVariant.GetAction().Index;
			NewAction = Definition.AddAction(EventHandle, InsertIndex);
		});

		return NewAction;
	}

	void FEventCollectionEditor::MoveDroppedAction(const FActionHandle& SourceAction, const FHandleVariant& TargetHandle, EItemDropZone DropZone)
	{
		using namespace UE::Subsonic::Core;

		if (TargetHandle.IsAction() && SourceAction == TargetHandle.GetAction())
		{
			return;
		}

		TransactEventCollection(LOCTEXT("MoveActionDescription", "Move action in Subsonic Event Collection"), [&](FSubsonicEventCollectionDefinition& Definition)
		{
			FSubsonicEvent* SourceEvent = Definition.FindMutableEvent(SourceAction.Event);
			if (!SourceEvent)
			{
				return;
			}

			TArray<FSubsonicEventActionDefinition>& SourceActions = SourceEvent->GetMutableActionCollection();
			if (!SourceActions.IsValidIndex(SourceAction.Index))
			{
				return;
			}

			FSubsonicEvent* TargetEvent = Definition.FindMutableEvent(TargetHandle.GetEvent());
			if (!TargetEvent)
			{
				return;
			}

			const int32 TargetIndex = TargetHandle.IsEvent()
				? TargetEvent->GetActionCollection().Num()
				: TargetHandle.GetAction().Index;

			FSubsonicEventActionDefinition MovedAction = MoveTemp(SourceActions[SourceAction.Index]);
			SourceActions.RemoveAt(SourceAction.Index);

			int32 InsertAt = INDEX_NONE;
			const int32 SourceIndex = SourceAction.Index;
			switch (DropZone)
			{
				case EItemDropZone::AboveItem:
				{
					InsertAt = (TargetIndex > SourceIndex) ? TargetIndex - 1 : TargetIndex;
				}
				break;

				case EItemDropZone::BelowItem:
				{
					InsertAt = (TargetIndex >= SourceIndex) ? TargetIndex : TargetIndex + 1;
				}
				break;

				case EItemDropZone::OntoItem:
				default:
				{
					InsertAt = TargetIndex;
				}
				break;
			}

			TArray<FSubsonicEventActionDefinition>& TargetActions = TargetEvent->GetMutableActionCollection();
			if (InsertAt != INDEX_NONE && TargetActions.Num() >= InsertAt)
			{
				TargetActions.Insert(MoveTemp(MovedAction), InsertAt);
			}
			else
			{
				TargetActions.Add(MoveTemp(MovedAction));

			}
		});

		RebuildEventTree(true /* bRebuildListView */);
	}

	void FEventCollectionEditor::BuildTransport()
	{
		TSharedRef<FExtender> ToolbarExtender = MakeShared<FExtender>();
		ToolbarExtender->AddToolBarExtension(
			"Asset",
			EExtensionHook::After,
			GetToolkitCommands(),
			FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
			{
				const FEventCollectionEditorCommands& Commands = FEventCollectionEditorCommands::Get();

				ToolbarBuilder.BeginSection("Transport");
				{
					ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateLeft");
					{
						ToolbarBuilder.AddToolBarButton(Commands.StartAudition, NAME_None, { }, { },
							TAttribute<FSlateIcon>::Create([this]()
							{
								if (!PreviewExecutor.IsValid())
								{
									static const FSlateIcon IconInactive = FSlateIcon("SubsonicStyle", "SubsonicEditor.Play.Inactive.Valid");
									return IconInactive;
								}

								static const FSlateIcon IconActive = FSlateIcon("SubsonicStyle", "SubsonicEditor.Play.Active.Valid");
								return IconActive;
							})
						);
					}
					ToolbarBuilder.EndStyleOverride();

					ToolbarBuilder.BeginStyleOverride("Toolbar.BackplateRight");
					{
						ToolbarBuilder.AddToolBarButton(Commands.StopAudition, NAME_None, { }, { },
							TAttribute<FSlateIcon>::Create([this]()
							{
								if (PreviewExecutor.IsValid())
								{
									static const FSlateIcon IconActive = FSlateIcon("SubsonicStyle", "SubsonicEditor.Stop.Active");
									return IconActive;
								}

								static const FSlateIcon IconInactive = FSlateIcon("SubsonicStyle", "SubsonicEditor.Stop.Inactive");
								return IconInactive;
							})
						);
					}
					ToolbarBuilder.EndStyleOverride();
				}
				ToolbarBuilder.EndSection();
			})
		);

		AddToolbarExtender(ToolbarExtender);
	}

	void FEventCollectionEditor::ExecutePreviewEvent(const FEventHandle& EventHandle)
	{
		if (!PreviewExecutor.IsValid())
		{
			RestartAudition(false /* bAutoEventsEnabled */);
		}

		if (PreviewExecutor.IsValid())
		{
			PreviewExecutor->GetExecutor().ExecuteEvent(EventHandle.EventName);

			// Record Lerp start time for the play button green Lerp animation
			EventExecuteLerpTimes.Add(EventHandle.EventName, FSlateApplication::Get().GetCurrentTime());

			// Register an active timer on the event tree to force repaints during the Lerp
			if (EventTreeView.IsValid())
			{
				EventTreeView->RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda(
					[this](double, float) -> EActiveTimerReturnType
					{
						const double Now = FSlateApplication::Get().GetCurrentTime();
						bool bAnyActive = false;
						for (auto It = EventExecuteLerpTimes.CreateIterator(); It; ++It)
						{
							if (Now - It->Value < EventCollectionEditorPrivate::IconLerpDuration)
							{
								bAnyActive = true;
							}
							else
							{
								It.RemoveCurrent();
							}
						}
						return bAnyActive ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
					}
				));
			}
		}
	}

	FName FEventCollectionEditor::GetEditorName() const
	{
		return FEventCollectionEditor::EditorName;
	}

	void FEventCollectionEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
	{
		using namespace UE::Subsonic::Core;

		if (EventTreeInspectorView.IsValid())
		{
			EventTreeInspectorView->ForceRefresh();
		}

		if (!PreviewExecutor.IsValid() || !TreeDetailsView.IsValid() || TreeDetailsView->ActionIndex < 0)
		{
			return;
		}

		const USubsonicEventCollection* Collection = GetEventCollection();
		if (!Collection)
		{
			return;
		}

		const FCollectionHandle CollectionHandle = Collection->GetHandle();
		const FActionHandle ActionHandle = {
			.Event = { .Collection = CollectionHandle, .EventName = TreeDetailsView->Event },
			.Index = TreeDetailsView->ActionIndex
		};

		const FSubsonicEventActionDefinition* ActionDef = Collection->GetDefinition().FindAction(ActionHandle);
		if (!ActionDef)
		{
			return;
		}

		const auto* PlayAction = ActionDef->GetAction().GetPtr<FSubsonicEventAction_GeneratorSourcePlay>();
		if (!PlayAction)
		{
			return;
		}

		const FExecutorScopeKey ScopeKey(PreviewExecutor->GetExecutor());
		USubsonicGeneratorSourceSubscriber* Subscriber = FindDeviceSubsystem<USubsonicGeneratorSourceSubscriber>(ScopeKey);
		if (!Subscriber)
		{
			return;
		}

		switch (PlayAction->Scope)
		{
		case ESubsonicExecutionScope::Global:
			Subscriber->SetAuthoredParameters(PlayAction->Name, PlayAction->Parameters);
			break;

		case ESubsonicExecutionScope::Executor:
			Subscriber->SetAuthoredParameters(ScopeKey, PlayAction->Name, PlayAction->Parameters);
			break;

		default:
			checkNoEntry();
		}
	}

	void FEventCollectionEditor::PostUndo(bool bSuccess)
	{
		FSlateApplication::Get().DismissAllMenus();
		RebuildEventTree(true /* bRebuildListView */);
		FEditorUndoClient::PostUndo(bSuccess);
	}

	void FEventCollectionEditor::RestartAudition(bool bAutoEventsEnabled)
	{
		if (const USubsonicEventCollection* EventCollection = GetEventCollection())
		{
			if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
			{
				if (FAudioDeviceHandle ActiveDevice = DeviceManager->GetActiveAudioDevice(); ActiveDevice.IsValid())
				{
					if (PreviewExecutor.IsValid())
					{
						PreviewExecutor->Unregister();
					}

					UObject* TransientObject = GetTransientPackageAsObject();
					check(TransientObject);
					PreviewExecutor.Reset(USubsonicEventCollectionExecutor::Create(*TransientObject, "PreviewExecutor", *EventCollection, ActiveDevice.GetDeviceID()));

					if (bAutoEventsEnabled)
					{
						const double LerpTime = FSlateApplication::Get().GetCurrentTime();
						const FSubsonicEventCollectionDefinition& CollectionDefinition = EventCollection->GetDefinition();
						for (const TPair<FGameplayTag, FSubsonicEvent>& Pair : CollectionDefinition.GetEvents())
						{
							if (Pair.Value.GetAutoAudition())
							{
								PreviewExecutor->GetExecutor().ExecuteEvent(Pair.Key.GetTagName());
								EventExecuteLerpTimes.Add(Pair.Key.GetTagName(), LerpTime);
							}
						}

						// Register an active timer to repaint during the Lerp
						if (EventExecuteLerpTimes.Num() > 0 && EventTreeView.IsValid())
						{
							EventTreeView->RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda(
								[this](double, float) -> EActiveTimerReturnType
								{
									const double Now = FSlateApplication::Get().GetCurrentTime();
									bool bAnyActive = false;
									for (auto It = EventExecuteLerpTimes.CreateIterator(); It; ++It)
									{
										if (Now - It->Value < EventCollectionEditorPrivate::IconLerpDuration)
										{
											bAnyActive = true;
										}
										else
										{
											It.RemoveCurrent();
										}
									}
									return bAnyActive ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
								}
							));
						}
					}
				}
			}
		}
	}

	void FEventCollectionEditor::StartAudition()
	{
		if (!PreviewExecutor.IsValid())
		{
			RestartAudition();
		}
	}

	void FEventCollectionEditor::StopAudition()
	{
		if (PreviewExecutor.IsValid())
		{
			PreviewExecutor->Unregister();
			PreviewExecutor.Reset();
		}
	}

	void FEventCollectionEditor::ToggleAudition()
	{
		if (PreviewExecutor.IsValid())
		{
			PreviewExecutor->Unregister();
			PreviewExecutor.Reset();
		}
		else
		{
			RestartAudition();
		}
	}
} // namespace UE::Subsonic::Editor
#undef LOCTEXT_NAMESPACE // SubsonicEditor
