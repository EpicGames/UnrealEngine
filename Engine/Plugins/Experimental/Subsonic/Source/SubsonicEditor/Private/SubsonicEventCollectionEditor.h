// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EditorUndoClient.h"
#include "Framework/Docking/TabManager.h"
#include "IDetailsView.h"
#include "Misc/NotifyHook.h"
#include "Misc/TVariant.h"
#include "SubsonicEventCollection.h"
#include "SubsonicEventCollectionObjects.h"
#include "SubsonicEventCollectionViews.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/StrongObjectPtrTemplates.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"


namespace UE::Subsonic
{
	namespace Editor
	{
		class FEventCollectionEditor : public FAssetEditorToolkit, public FNotifyHook, public FEditorUndoClient
		{
		public:
			using FActionHandle = Core::FActionHandle;
			using FEventHandle = Core::FEventHandle;
			using FCollectionHandle = Core::FCollectionHandle;

			using FSubsonicEvent = Core::FSubsonicEvent;
			using FSubsonicEventActionDefinition = Core::FSubsonicEventActionDefinition;
			using FSubsonicEventCollectionDefinition = Core::FSubsonicEventCollectionDefinition;

			static const FName EditorName;

			FEventCollectionEditor() = default;
			virtual ~FEventCollectionEditor();

			virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
			virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

			void Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USubsonicEventCollection& EventCollection);

			USubsonicEventCollection* GetEventCollection() const;

			// Selects the event row (not an action row) in the event tree for the given event name.
			void SelectEventInTree(FName EventName);

			void TryInvokeTab(FName TabName) const;

			/** IToolkit Interface */
			virtual FName GetToolkitFName() const override;
			virtual FText GetBaseToolkitName() const override;
			virtual FString GetWorldCentricTabPrefix() const override;
			virtual FLinearColor GetWorldCentricTabColorScale() const override;
			virtual const FSlateBrush* GetDefaultTabIcon() const override;
			virtual FLinearColor GetDefaultTabColor() const override;

			/** IAssetEditorInstance Interface */
			virtual FName GetEditorName() const override;

			/** FEditorUndoClient Interface */
			virtual void PostUndo(bool bSuccess) override;
			virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

			/** FNotifyHook Interface */
			virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

		private:
			void TransactEvent(const Core::FEventHandle EventHandle, FText Description, TFunctionRef<void(FSubsonicEvent&)> TransactionFunc) const;
			void TransactEventCollection(FText Description, TFunctionRef<void(FSubsonicEventCollectionDefinition&)> TransactionFunc) const;

			TSharedRef<SWidget> InitEventTreeView();

			class FHandleVariant
			{
				TVariant<Core::FEventHandle, Core::FActionHandle> Handle;

			public:
				const Core::FActionHandle& GetAction() const
				{
					checkf(IsAction(), TEXT("Must check that handle is action first"));
					return Handle.Get<Core::FActionHandle>();
				}

				const Core::FEventHandle& GetEvent() const
				{
					return Handle.IsType<Core::FActionHandle>()
						? Handle.Get<Core::FActionHandle>().Event
						: Handle.Get<Core::FEventHandle>();
				}

				bool IsAction() const
				{
					return Handle.IsType<Core::FActionHandle>();
				}

				bool IsEvent() const
				{
					return Handle.IsType<Core::FEventHandle>();
				}

				template <typename HandleType>
				void Set(HandleType NewHandle)
				{
					Handle.Set<HandleType>(MoveTemp(NewHandle));
				}
			};

			// Inserts action. If the handle variant points at an event, adds to end of event array.
			// If it points to an action, inserts before the given action.
			Core::FSubsonicEventActionDefinition* InsertAction(const FHandleVariant& InHandleVariant);

			// Moves an action to the position relative to TargetAction per the given DropZone.
			void MoveDroppedAction(const Core::FActionHandle& SourceAction, const FHandleVariant& HandleVariant, EItemDropZone DropZone);

			void BuildTransport();

			void ExecutePreviewEvent(const Core::FEventHandle& EventHandle);

			void InitBindings();

			using SHandleVariantTableRow = STableRow<TSharedPtr<FHandleVariant>>;

			// Currently, only supports actions. Could support dragging and ordering events in specialized view sorted by user-defined index in the future.
			TSharedRef<SHandleVariantTableRow> InitDraggableEventRow(TSharedPtr<FHandleVariant> HandleVariant, const TSharedRef<STableViewBase>& TableView);

			TSharedRef<ITableRow> OnEventTree_GenerateRow(TSharedPtr<FHandleVariant> HandleVariant, const TSharedRef<STableViewBase>& TableView);
			void OnEventTree_GetChildren(TSharedPtr<FHandleVariant> InParent, TArray<TSharedPtr<FHandleVariant>>& OutChildren);
			void OnEventTree_RowSelected(TSharedPtr<FHandleVariant> HandleVariant, ESelectInfo::Type SelectInfo);

			void ExpandAllEvents();
			void CollapseAllEvents();

			void RebuildEventTree(bool bRebuildListView = false);

			void RestartAudition(bool bAutoEventsEnabled = true);
			void StartAudition();
			void StopAudition();
			void ToggleAudition();

			TStrongObjectPtr<USubsonicEventCollectionExecutor> PreviewExecutor;

			TSharedPtr<IDetailsView> EventTreeInspectorView;
			TSharedPtr<FTabManager::FStack> DetailsStack;
			TStrongObjectPtr<USubsonicEventTreeDetailsView> TreeDetailsView;

			TSharedPtr<FTabManager::FStack> EventTreeStack;

			TSharedPtr<IDetailsView> ParametersDetailsView;
			TSharedPtr<FTabManager::FStack> ParametersStack;
			TStrongObjectPtr<USubsonicCollectionParametersView> ParametersView;

			TArray<TSharedPtr<FHandleVariant>> EventTreeRootHandles;
			TMap<FName, TArray<TSharedPtr<FHandleVariant>>> EventTreeChildHandles;
			TSharedPtr<STreeView<TSharedPtr<FHandleVariant>>> EventTreeView;
			FString EventViewFilter;

			// Value names of selectable actions in event tree view
			TArray<TSharedPtr<FString>> ActionComboValues;

			// Single property view of collection for use to capture bespoke property handles
			// for visualization in contexts outside of a contextual details layout/panel
			TSharedPtr<ISinglePropertyView> CollectionView;

			FDelegateHandle StaleBindingsDelegateHandle;

			// Tracks the last time each event was test-executed via the play button,
			// used to Lerp the button green and fade back to white.
			TMap<FName, double> EventExecuteLerpTimes;
		};
	} // namespace Editor
} // namespace UE::Subsonic