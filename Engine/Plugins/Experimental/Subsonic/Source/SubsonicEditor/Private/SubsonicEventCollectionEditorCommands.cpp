// Copyright Epic Games, Inc. All Rights Reserved.
#include "SubsonicEventCollectionEditorCommands.h"

#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"


#define LOCTEXT_NAMESPACE "SubsonicEditor"
namespace UE::Subsonic::Editor
{
	void FEventCollectionEditorCommands::RegisterCommands()
	{
		UI_COMMAND(StartAudition, "Start Audition", "Creates and initalizes Subsonic audition executor", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(StopAudition, "Stop Audition", "Destroys the audition executor, clearing associate subscriber state", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ToggleAudition, "Toggle Audition", "Starts or stops the currently auditioning executor", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar));
	}
} // namespace UE::Subsonic::Editor
#undef LOCTEXT_NAMESPACE // SubsonicEditor
