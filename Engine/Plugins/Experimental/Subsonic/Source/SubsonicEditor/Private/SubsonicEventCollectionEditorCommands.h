// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::Subsonic::Editor
{
	class FEventCollectionEditorCommands : public TCommands<FEventCollectionEditorCommands>
	{
	public:
		FEventCollectionEditorCommands()
			: TCommands<FEventCollectionEditorCommands>("SubsonicEditor", NSLOCTEXT("SubsonicEditor", "SubsonicEditorCommands_EditorName", "Subsonic Event Collection Editor"), NAME_None, "SubsonicStyle")
		{
		}

		/** Creates and initializes Subsonic audition executor */
		TSharedPtr<FUICommandInfo> StartAudition;

		/** Destroys the auditioning executor, clearing associate subscriber state. */
		TSharedPtr<FUICommandInfo> StopAudition;

		/** Starts or stops the currently auditioning executor */
		TSharedPtr<FUICommandInfo> ToggleAudition;

		/** Initialize commands */
		virtual void RegisterCommands() override;
	};
} // namespace UE::Subsonic::Editor