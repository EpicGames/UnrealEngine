// Copyright Epic Games, Inc. All Rights Reserved.
#include "SubsonicEventCollectionFactory.h"

#include "SubsonicEventCollectionObjects.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubsonicEventCollectionFactory)


namespace UE::Subsonic::Editor
{
	USubsonicEventCollectionFactory::USubsonicEventCollectionFactory(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		bCreateNew = true;
		bEditorImport = false;
		bEditAfterNew = true;

		SupportedClass = USubsonicEventCollection::StaticClass();
	}

	UObject* USubsonicEventCollectionFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InFeedbackContext)
	{
		return NewObject<USubsonicEventCollection>(InParent, InClass, InName, InFlags);
	}
} // namespace UE::Subsonic::Editor
