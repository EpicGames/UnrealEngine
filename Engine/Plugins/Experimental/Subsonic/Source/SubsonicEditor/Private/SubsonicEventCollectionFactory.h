// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "UObject/Object.h"

#include "SubsonicEventCollectionObjects.h"

#include "SubsonicEventCollectionFactory.generated.h"


namespace UE::Subsonic::Editor
{
	UCLASS()
		class USubsonicEventCollectionFactory : public UFactory
	{
		GENERATED_UCLASS_BODY()

		//~ Begin UFactory Interface
		virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InFeedbackContext) override;
		//~ Begin UFactory Interface
	};
} // namespace UE::Subsonic::Editor
