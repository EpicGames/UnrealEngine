// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicEventCollectionViews.h"

#include "DetailCategoryBuilder.h"
#include "Internationalization/Internationalization.h"
#include "SubsonicEventCollection.h"
#include "SubsonicEventCollectionObjects.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubsonicEventCollectionViews)

#define LOCTEXT_NAMESPACE "SubsonicEditor"


namespace UE::Subsonic::Editor
{
	namespace CollectionViewsPrivate
	{
		void AddActionProperties(IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder, TSharedRef<IPropertyHandleArray> ActionArrayHandle, uint32 ActionIndex)
		{
			using namespace UE::Subsonic::Core;

			TSharedRef<IPropertyHandle> ActionEntryHandle = ActionArrayHandle->GetElement(ActionIndex);
			uint32 NumProperties = INDEX_NONE;
			if (ActionEntryHandle->GetNumChildren(NumProperties) == FPropertyAccess::Success)
			{
				for (uint32 Index = 0; Index < NumProperties; ++Index)
				{
					TSharedPtr<IPropertyHandle> ChildHandle = ActionEntryHandle->GetChildHandle(Index);
					if (ChildHandle.IsValid())
					{
						const FName Category = ChildHandle->GetDefaultCategoryName();
						Category.IsNone() ? CategoryBuilder.AddProperty(ChildHandle) : LayoutBuilder.EditCategory(Category).AddProperty(ChildHandle);
					}
				}
			}
		}

		TSharedPtr<IPropertyHandle> GetEventHandleMatchingName(TSharedRef<IPropertyHandleMap> EventMapHandle, uint32 Index, FName Name)
		{
			using namespace UE::Subsonic::Core;

			TSharedRef<IPropertyHandle> EventEntryHandle = EventMapHandle->GetElement(Index);
			if (TSharedPtr<IPropertyHandle> EventKeyHandle = EventEntryHandle->GetKeyHandle(); EventKeyHandle.IsValid())
			{
				FName EventName;
				TSharedPtr<IPropertyHandle> TagNameHandle = EventKeyHandle->GetChildHandle("TagName");
				if (TagNameHandle.IsValid()
					&& TagNameHandle->GetValue(EventName) == FPropertyAccess::Success
					&& EventName == Name)
				{
					return EventEntryHandle;
				}
			}

			return { };
		}
	} // ViewsPrivate

	void FCollectionParametersDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
	{
		using namespace UE::Subsonic::Core;

		if (USubsonicEventCollectionViewBase* View = GetViewFromBuilder(LayoutBuilder))
		{
			const FName ParametersPropertyName = FSubsonicEventCollectionDefinition::GetParametersPropertyName();
			TSharedPtr<IPropertyHandle> ChildHandle = CacheCollectionPropertyHandle(LayoutBuilder, *View, ParametersPropertyName);
			if (ChildHandle.IsValid())
			{
				const FText CategoryDisplayName = HeaderName.IsEmptyOrWhitespace() ? FText::FromName(ParametersPropertyName) : HeaderName;
				IDetailCategoryBuilder& CategoryBuilder = LayoutBuilder.EditCategory(ParametersPropertyName, CategoryDisplayName);
				CategoryBuilder.InitiallyCollapsed(false);

				const FName Category = ChildHandle->GetDefaultCategoryName();
				Category.IsNone() ? CategoryBuilder.AddProperty(ChildHandle) : LayoutBuilder.EditCategory(Category).AddProperty(ChildHandle);
			}
		}
	}

	TSharedPtr<IPropertyHandle> FCollectionPropertyCustomizationBase::CacheCollectionPropertyHandle(
		IDetailLayoutBuilder& LayoutBuilder,
		USubsonicEventCollectionViewBase& View,
		FName ChildPropertyName)
	{
		if (USubsonicEventCollection* ViewCollection = View.GetCollection())
		{
			TSharedPtr<IPropertyHandle> DefinitionHandle = LayoutBuilder.AddObjectPropertyData(
				TArray<UObject*> { ViewCollection },
				USubsonicEventCollection::GetDefinitionPropertyName());

			if (DefinitionHandle.IsValid())
			{
				return DefinitionHandle->GetChildHandle(ChildPropertyName);
			}
		}

		return { };
	}

	USubsonicEventCollectionViewBase* FCollectionPropertyCustomizationBase::GetViewFromBuilder(IDetailLayoutBuilder& LayoutBuilder)
	{
		// Only supports modifying a single collection view at a time
		TArray<TWeakObjectPtr<UObject>> Objects;
		LayoutBuilder.GetObjectsBeingCustomized(Objects);
		if (Objects.Num() == 1 && Objects.Last().IsValid())
		{
			return CastChecked<USubsonicEventCollectionViewBase>(Objects.Last());
		}

		return nullptr;
	}

	FCollectionParametersDetailCustomization::FCollectionParametersDetailCustomization()
	{
		HeaderName = LOCTEXT("ParametersDetails_DisplayName", "Parameters");
	}

	FEventTreeSelectionDetailCustomization::FEventTreeSelectionDetailCustomization()
	{
		HeaderName = LOCTEXT("EventTreeDetails_DisplayName", "Details");
	}

	void FEventTreeSelectionDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
	{
		using namespace UE::Subsonic::Core;

		USubsonicEventTreeDetailsView* View = Cast<USubsonicEventTreeDetailsView>(GetViewFromBuilder(LayoutBuilder));
		if (!View)
		{
			return;
		}

		TSharedPtr<IPropertyHandle> ChildProperty = CacheCollectionPropertyHandle(LayoutBuilder, *View, FSubsonicEventCollectionDefinition::GetEventsPropertyName());
		if (!ChildProperty.IsValid())
		{
			return;
		}

		TSharedPtr<IPropertyHandleMap> EventMapHandle = ChildProperty->AsMap();
		if (!EventMapHandle.IsValid())
		{
			return;
		}

		uint32 NumEvents = 0;
		if (EventMapHandle->GetNumElements(NumEvents) == FPropertyAccess::Success && NumEvents > 0)
		{
			const FText CategoryDisplayName = HeaderName.IsEmptyOrWhitespace() ? FText::FromName(FSubsonicEventCollectionDefinition::GetEventsPropertyName()) : HeaderName;
			IDetailCategoryBuilder& CategoryBuilder = LayoutBuilder.EditCategory(ChildProperty->GetDefaultCategoryName(), CategoryDisplayName);
			CategoryBuilder.InitiallyCollapsed(false);

			for (uint32 EventIndex = 0; EventIndex < NumEvents; ++EventIndex)
			{
				const FName ActionCollectionPropertyName = FSubsonicEvent::GetActionCollectionPropertyName();
				if (TSharedPtr<IPropertyHandle> EventHandle = CollectionViewsPrivate::GetEventHandleMatchingName(EventMapHandle.ToSharedRef(), EventIndex, View->Event))
				{
					// Show Event Properties (View->ActionIndex is INDEX_NONE/-1)
					if (View->ActionIndex < 0)
					{
						uint32 NumChildren = 0;
						EventHandle->GetNumChildren(NumChildren);
						for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
						{
							if (TSharedPtr<IPropertyHandle> ChildHandle = EventHandle->GetChildHandle(ChildIndex))
							{
								if (FProperty* Property = ChildHandle->GetProperty(); Property && Property->GetFName() != ActionCollectionPropertyName)
								{
									const FName Category = ChildHandle->GetDefaultCategoryName();
									Category.IsNone() ? CategoryBuilder.AddProperty(ChildHandle) : LayoutBuilder.EditCategory(Category).AddProperty(ChildHandle);
								}
							}
						}
					}
					// Show Action (Selected) Properties
					else
					{
						if (TSharedPtr<IPropertyHandle> ActionCollectionHandle = EventHandle->GetChildHandle(ActionCollectionPropertyName, false /* bRecurse */))
						{
							if (TSharedPtr<IPropertyHandleArray> ArrayHandle = ActionCollectionHandle->AsArray())
							{
								uint32 NumActions = 0;
								if (ArrayHandle->GetNumElements(NumActions) == FPropertyAccess::Success && static_cast<int32>(NumActions) > View->ActionIndex)
								{
									CollectionViewsPrivate::AddActionProperties(LayoutBuilder, CategoryBuilder, ArrayHandle.ToSharedRef(), View->ActionIndex);
								}
							}
						}
					}
				}
			}
		}
	}
} // namespace UE::Subsonic::Editor
#undef LOCTEXT_NAMESPACE // "SubsonicEditor"
