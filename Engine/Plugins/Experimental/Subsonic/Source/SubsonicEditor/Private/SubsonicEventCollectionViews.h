// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "SubsonicEventCollection.h"
#include "SubsonicEventCollectionObjects.h"

#include "SubsonicEventCollectionViews.generated.h"


namespace UE::Subsonic::Editor
{
	// Base implementation for Subsonic Collection view wrappers that coincide with edit customizations
	UCLASS(MinimalAPI)
	class USubsonicEventCollectionViewBase : public UObject
	{
		GENERATED_BODY()

	public:

		void SetCollection(USubsonicEventCollection* InCollection) { Collection = InCollection; }
		const USubsonicEventCollection* GetConstCollection() const { return Collection.Get(); }
		USubsonicEventCollection* GetCollection() { return Collection.Get(); }

	private:
		TWeakObjectPtr<USubsonicEventCollection> Collection;
	};

	UCLASS(MinimalAPI)
	class USubsonicCollectionParametersView : public USubsonicEventCollectionViewBase
	{
		GENERATED_BODY()
	};

	UCLASS(MinimalAPI)
	class USubsonicEventTreeDetailsView : public USubsonicEventCollectionViewBase
	{
		GENERATED_BODY()

	public:
		FName Event;
		int32 ActionIndex = INDEX_NONE;
	};

	class FCollectionPropertyCustomizationBase : public IDetailCustomization
	{
	protected:
		FText HeaderName;

	protected:
		static TSharedPtr<IPropertyHandle> CacheCollectionPropertyHandle(
			IDetailLayoutBuilder& LayoutBuilder,
			USubsonicEventCollectionViewBase& View,
			FName ChildPropertyName);

		static USubsonicEventCollectionViewBase* GetViewFromBuilder(IDetailLayoutBuilder& LayoutBuilder);
	};

	class FCollectionParametersDetailCustomization : public FCollectionPropertyCustomizationBase
	{
	public:
		FCollectionParametersDetailCustomization();

		// IDetailCustomization interface
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
		// End of IDetailCustomization interface
	};

	class FEventTreeSelectionDetailCustomization : public FCollectionPropertyCustomizationBase
	{
	public:
		FEventTreeSelectionDetailCustomization();

		virtual void CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder) override;
	};
} // namespace UE::Subsonic::Editor