// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicPropertyBindingExtension.h"

#include "Algo/AnyOf.h"
#include "EdGraphSchema_K2.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SubsonicBindingUtils.h"
#include "SubsonicEventCollection.h"
#include "SubsonicEventCollectionViews.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "SubsonicEditor"

namespace UE::Subsonic::Editor
{
	FSubsonicPropertyBindingExtension::FSubsonicPropertyBindingExtension(FNavigateToPropertyOwnerFunc InNavigateToPropertyOwner)
		: NavigateToPropertyOwner(MoveTemp(InNavigateToPropertyOwner))
	{
	}

	bool FSubsonicPropertyBindingExtension::IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const
	{
		const FProperty* Property = InPropertyHandle.GetProperty();
		if (!Property || Property->HasMetaData("NoBinding"))
		{
			return false;
		}

		// Skip the base class struct-ID check; we supply our own binding context below.
		return true;
	}

	FPropertyBindingWidgetArgs FSubsonicPropertyBindingExtension::InitWidgetRow(
		USubsonicEventCollection& Collection,
		USubsonicEventTreeDetailsView& View,
		const FProperty* TargetProperty,
		FDetailWidgetRow& WidgetRow)
	{
		using namespace UE::Subsonic::Core;

		check(TargetProperty);

		// Weak refs used inside all delegates to guard against stale pointers.
		TWeakObjectPtr<USubsonicEventCollection> WeakCollection(&Collection);
		TWeakObjectPtr<USubsonicEventTreeDetailsView> WeakView(&View);

		// Returns which parameter name (if any) is currently bound to PropertyName.
		auto GetCurrentBindingParam = [WeakCollection, WeakView, TargetProperty]()
		{
			const USubsonicEventCollection* Collection = WeakCollection.Get();
			const USubsonicEventTreeDetailsView* DetailsView = WeakView.Get();
			if (Collection && DetailsView)
			{
				if (const FSubsonicEvent* Event = Collection->GetDefinition().FindEvent(DetailsView->Event))
				{
					const int32 Index = DetailsView->ActionIndex;
					const TArray<FSubsonicEventActionDefinition>& ActionCollection = Event->GetActionCollection();
					if (ActionCollection.IsValidIndex(Index))
					{
						if (const FSubsonicEventActionBase* ActionBase = ActionCollection[Index].GetAction().GetPtr())
						{
							const FName PropertyName = TargetProperty->GetFName();
							return ActionBase->GetBoundParameterForProperty(PropertyName);
						}
					}
				}
			}
			return FName();
		};

		TSharedRef<SWidget> ValueWidget = WidgetRow.ValueContent().Widget;
		WidgetRow.ValueContent()
		[
			SNew(SBox)
			.IsEnabled_Lambda([GetCurrentBindingParam]()
			{
				return GetCurrentBindingParam().IsNone();
			})
			.ToolTip_Lambda([GetCurrentBindingParam, ValueWidget]()
			{
				const FName CurrentParam = GetCurrentBindingParam();
				return CurrentParam.IsNone()
					? ValueWidget->GetToolTip()
					: SNew(SToolTip).Text(FText::Format(
						LOCTEXT("NotEditableWhenBound_Tooltip", "Parameter is currently bound to {0} (Remove binding to directly edit)"),
						FText::FromName(CurrentParam)));
			})
			[
				ValueWidget
			]
		];

		FPropertyBindingWidgetArgs Args;
		// FPropertyBindingWidgetArgs::Property is not const-correct; the binding widget only reads the property.
		Args.Property = const_cast<FProperty*>(TargetProperty);
		Args.bAllowNewBindings = false;
		Args.bAllowFunctionBindings = false;
		Args.bAllowUObjectFunctions = false;
		Args.bAllowStructMemberBindings = false;
		Args.BindButtonStyle = &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("HoverHintOnly");

		Args.OnGenerateMenu = FOnGetContent::CreateLambda(
			[this, TargetProperty, GetCurrentBindingParam, WeakCollection, WeakView]()
			{
				return GenerateBindingMenu(TargetProperty, GetCurrentBindingParam, WeakCollection, WeakView);
			});

		Args.OnRemoveBinding = FOnRemoveBinding::CreateLambda(
			[WeakCollection, WeakView, TargetProperty](FName /*InPropertyName*/)
			{
				RemoveBinding(WeakCollection, WeakView, TargetProperty);
			});

		Args.OnCanRemoveBinding = FOnCanRemoveBinding::CreateLambda([GetCurrentBindingParam](FName)
		{
			return !GetCurrentBindingParam().IsNone();
		});

		Args.OnHasAnyBindings = FOnHasAnyBindings::CreateLambda([GetCurrentBindingParam]()
		{
			return !GetCurrentBindingParam().IsNone();
		});

		Args.CurrentBindingText = TAttribute<FText>::Create([GetCurrentBindingParam, WeakCollection, WeakView]()
		{
			return GetBindingText(GetCurrentBindingParam, WeakCollection, WeakView);
		});

		return Args;
	}

	TSharedRef<SWidget> FSubsonicPropertyBindingExtension::GenerateBindingMenu(
		const FProperty* TargetProperty,
		TFunction<FName()> GetCurrentBindingParam,
		TWeakObjectPtr<USubsonicEventCollection> WeakCollection,
		TWeakObjectPtr<USubsonicEventTreeDetailsView> WeakView)
	{
		using namespace UE::Subsonic::Core;

		constexpr bool bInShouldCloseWindowAfterMenuSelection = true;
		TSharedPtr<const FUICommandList> NullCommandList;
		FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, NullCommandList);

		const USubsonicEventCollection* Collection = WeakCollection.Get();
		const USubsonicEventTreeDetailsView* DetailsView = WeakView.Get();
		if (!Collection || !DetailsView)
		{
			return MenuBuilder.MakeWidget();
		}

		// Rebuild binding contexts from live data each time the menu opens so that
		// renamed / removed parameters are never shown with stale names.
		constexpr FSlateBrush* NullIconBrush = nullptr;
		TArray<FSubsonicBindingContextStruct> BindingContextStructs;
		const FSubsonicEventCollectionDefinition& CollectionDef = Collection->GetDefinition();

		if (const UPropertyBag* CollectionBag = CollectionDef.GetParameters().GetPropertyBagStruct())
		{
			const FText Desc = LOCTEXT("BindingContext_CollectionDescription", "Collection");
			BindingContextStructs.Add(FSubsonicBindingContextStruct(CollectionBag, NullIconBrush, Desc));
		}

		if (const FSubsonicEvent* Event = CollectionDef.FindEvent(DetailsView->Event))
		{
			if (const UPropertyBag* EventBag = Event->GetParameters().GetPropertyBagStruct())
			{
				const FText Desc = LOCTEXT("BindingContext_EventDescription", "Event");
				FSubsonicBindingContextStruct Context(EventBag, NullIconBrush, Desc);
				Context.bIsEventBinding = true;
				BindingContextStructs.Add(MoveTemp(Context));
			}
		}

		struct FStructBindingEntry
		{
			int32 Index = INDEX_NONE;
			FProperty* BagProp = nullptr;
		};

		TArray<FName> OrderedNames;
		TMap<FName, FStructBindingEntry> EntryMap;

		for (int32 Index = 0; Index < BindingContextStructs.Num(); ++Index)
		{
			const FBindingContextStruct& BindingContext = BindingContextStructs[Index];
			for (TFieldIterator<FProperty> PropIt(BindingContext.Struct); PropIt; ++PropIt)
			{
				FProperty* BagProp = *PropIt;
				if (!BagProp || !BindingUtils::ArePropertiesBindingCompatible(BagProp, TargetProperty))
				{
					continue;
				}
				const FName BagPropName = BagProp->GetFName();
				if (!EntryMap.Contains(BagPropName))
				{
					OrderedNames.Add(BagPropName);
				}

				// Overwrite any earlier context entry with event context (Event > Collection).
				EntryMap.Emplace(BagPropName, FStructBindingEntry { Index, BagProp });
			}
		}

		Algo::Sort(OrderedNames, [](const FName& A, const FName& B) { return A.LexicalLess(B); });

		for (const FName& BagPropName : OrderedNames)
		{
			const FStructBindingEntry& Entry = EntryMap[BagPropName];
			const int32 Index = Entry.Index;
			const FSubsonicBindingContextStruct& BindingContext = BindingContextStructs[Index];
			FProperty* BagProp = Entry.BagProp;
			const FText Tooltip = FText::Format(LOCTEXT("BindingSourceLevel_Tooltip", "Defined in: {0}"),
				BindingContext.DisplayText);

			// For contexts that have a navigation target, the tooltip is an interactive
			// hyperlink that selects the source in the editor when clicked. For other contexts
			// (Collection) it is plain text.
			TSharedRef<FCheckBoxStyle> EntryStyle = GetCheckboxEntryStyle(*BagProp);
			TSharedRef<SToolTip> EntryTooltip = SNew(SToolTip)
				.IsInteractive(true)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.BrightBackground"))
				.TextMargin(FMargin(11.0f))
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				[
					SNew(SHyperlink)
					.Text(Tooltip)
					.Style(FAppStyle::Get(), "DarkHyperlink")
					.OnNavigate_Lambda([NavigateToPropertyOwner = NavigateToPropertyOwner, bIsEvent = BindingContext.bIsEventBinding, WeakView]()
					{
						if (const USubsonicEventTreeDetailsView* View = WeakView.Get())
						{
							const FName BagEventName = bIsEvent ? View->Event : FName();
							NavigateToPropertyOwner(BagEventName);
						}
					})
				];

			MenuBuilder.AddWidget(
				SNew(SBox)
				.Padding(FMargin(30.0f, 2.0f, 4.0f, 4.0f))
				[
					SNew(SCheckBox)
					.Style(&EntryStyle.Get())
					.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
					.ToolTip(EntryTooltip)
					.IsChecked_Lambda([GetCurrentBindingParam, BagPropName, EntryStyle]()
					{
						return GetCurrentBindingParam() == BagPropName
							? ECheckBoxState::Checked
							: ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([WeakCollection, WeakView, TargetProperty, Index, BagProp, GetCurrentBindingParam, BagPropName](ECheckBoxState)
					{
						// Always close the menu on click (radio button semantics).
						// Only re-bind if selecting a different parameter to avoid a spurious undo entry.
						if (GetCurrentBindingParam() != BagPropName)
						{
							AddParamBinding(WeakCollection.Get(), WeakView.Get(), TargetProperty->GetFName(), Index, BagProp);
						}
						FSlateApplication::Get().DismissAllMenus();
					})
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "Menu.Label")
						.Text(FText::FromName(BagPropName))
					]
				],
				FText::GetEmpty()
			);
		}

		const FName BoundParam = GetCurrentBindingParam();
		if (!BoundParam.IsNone())
		{
			MenuBuilder.AddMenuSeparator();
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RemoveBinding", "Remove Binding"),
				FText::GetEmpty(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.X"),
				FUIAction(FExecuteAction::CreateLambda(
					[WeakCollection, WeakView, TargetProperty]()
					{
						RemoveBinding(WeakCollection, WeakView, TargetProperty);
					})
				)
			);
		}
		return MenuBuilder.MakeWidget();
	}

	FText FSubsonicPropertyBindingExtension::GetBindingText(
		TFunction<FName()> GetCurrentBindingParam,
		TWeakObjectPtr<USubsonicEventCollection> WeakCollection,
		TWeakObjectPtr<USubsonicEventTreeDetailsView> WeakView)
	{
		using namespace UE::Subsonic::Core;

		const FName BoundParam = GetCurrentBindingParam();
		if (BoundParam.IsNone())
		{
			return FText();
		}

		// Resolve from live data to avoid stale bag pointers. Check Event first
		// since event-level parameters take precedence over collection-level.
		const USubsonicEventCollection* Collection = WeakCollection.Get();
		const USubsonicEventTreeDetailsView* DetailsView = WeakView.Get();
		if (Collection && DetailsView)
		{
			const FSubsonicEventCollectionDefinition& Def = Collection->GetDefinition();
			if (const FSubsonicEvent* Event = Def.FindEvent(DetailsView->Event))
			{
				if (const UPropertyBag* EventBag = Event->GetParameters().GetPropertyBagStruct())
				{
					if (EventBag->FindPropertyByName(BoundParam))
					{
						return FText::Format(LOCTEXT("BoundParameter_DescFormat", "{0} > {1}"),
							LOCTEXT("BindingContext_EventDescription", "Event"), FText::FromName(BoundParam));
					}
				}
			}
			if (const UPropertyBag* CollBag = Def.GetParameters().GetPropertyBagStruct())
			{
				if (CollBag->FindPropertyByName(BoundParam))
				{
					return FText::Format(LOCTEXT("BoundParameter_DescFormat", "{0} > {1}"),
						LOCTEXT("BindingContext_CollectionDescription", "Collection"), FText::FromName(BoundParam));
				}
			}
		}
		return FText::FromName(BoundParam);
	}

	void FSubsonicPropertyBindingExtension::RemoveBinding(
		TWeakObjectPtr<USubsonicEventCollection> WeakCollection,
		TWeakObjectPtr<USubsonicEventTreeDetailsView> WeakView,
		const FProperty* TargetProperty)
	{
		using namespace UE::Subsonic::Core;

		USubsonicEventCollection* Collection = WeakCollection.Get();
		USubsonicEventTreeDetailsView* DetailsView = WeakView.Get();
		if (Collection && DetailsView)
		{
			const int32 Index = DetailsView->ActionIndex;
			const FEventHandle EventHandle { .Collection = Collection->GetHandle(), .EventName = DetailsView->Event };
			FSubsonicEvent* Event = Collection->GetMutableDefinition().FindMutableEvent(EventHandle);
			if (Event && Event->GetActionCollection().IsValidIndex(Index))
			{
				if (FSubsonicEventActionBase* ActionBase = Event->GetMutableActionCollection()[Index].GetMutableActionBase())
				{
					TransactEventCollection(LOCTEXT("RemoveSubsonicBinding", "Remove Subsonic Parameter Binding"), *Collection, [&](FSubsonicEventCollectionDefinition&)
					{
						const FName PropertyName = TargetProperty->GetFName();
						ActionBase->RemovePropertyBinding(PropertyName);
					});
				}
			}
		}
	}

	void FSubsonicPropertyBindingExtension::ExtendWidgetRow(FDetailWidgetRow& WidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> InPropertyHandle)
	{
		using namespace UE::Subsonic::Core;

		if (!IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
		{
			return;
		}

		// Resolve the inspector view so we know which event / action is selected.
		TArray<TWeakObjectPtr<UObject>> Objects;
		InDetailBuilder.GetObjectsBeingCustomized(Objects);
		if (Objects.Num() != 1)
		{
			return;
		}

		USubsonicEventTreeDetailsView* View = Cast<USubsonicEventTreeDetailsView>(Objects.Last().Get());
		if (!View || View->ActionIndex < 0)
		{
			return;
		}

		USubsonicEventCollection* Collection = View->GetCollection();
		if (!Collection)
		{
			return;
		}

		const FProperty* TargetProperty = InPropertyHandle->GetProperty();
		if (!TargetProperty)
		{
			return;
		}

		TArray<FBindingContextStruct> BindingContextStructs;
		const FSubsonicEventCollectionDefinition& CollectionDef = Collection->GetDefinition();
		constexpr FSlateBrush* NullIconBrush = nullptr;

		if (const UPropertyBag* CollectionBag = CollectionDef.GetParameters().GetPropertyBagStruct())
		{
			const FText BindingContextDesc = LOCTEXT("BindingContext_CollectionDescription", "Collection");
			BindingContextStructs.Add(FSubsonicBindingContextStruct(CollectionBag, NullIconBrush, BindingContextDesc));
		}

		if (const FSubsonicEvent* Event = CollectionDef.FindEvent(View->Event))
		{
			if (const UPropertyBag* EventBag = Event->GetParameters().GetPropertyBagStruct())
			{
				const FText BindingContextDesc = LOCTEXT("BindingContext_EventDescription", "Event");
				BindingContextStructs.Add(FSubsonicBindingContextStruct(EventBag, NullIconBrush, BindingContextDesc));
			}
		}

		// Only show the binding dropdown if at least one compatible parameter exists.
		const bool bHasCompatibleParam = Algo::AnyOf(BindingContextStructs, [&TargetProperty](const FBindingContextStruct& Context)
		{
			for (TFieldIterator<FProperty> PropIt(Context.Struct); PropIt; ++PropIt)
			{
				if (*PropIt && BindingUtils::ArePropertiesBindingCompatible(*PropIt, TargetProperty))
				{
					return true;
				}
			}
			return false;
		});
		if (bHasCompatibleParam)
		{
			FPropertyBindingWidgetArgs Args = InitWidgetRow(*Collection, *View, TargetProperty, WidgetRow);
			IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
			WidgetRow.ExtensionContent()
			[
				PropertyAccessEditor.MakePropertyBindingWidget(BindingContextStructs, Args)
			];
		}
	}

	void FSubsonicPropertyBindingExtension::AddParamBinding(USubsonicEventCollection* Collection, USubsonicEventTreeDetailsView* DetailsView, FName PropertyName, const int32 SourceStructIndex, const FProperty* SourceProperty)
	{
		using namespace UE::Subsonic::Core;

		if (!SourceProperty || !Collection || !DetailsView || DetailsView->ActionIndex < 0)
		{
			return;
		}

		const FEventHandle EventHandle { .Collection = Collection->GetHandle(), .EventName = DetailsView->Event };
		FSubsonicEvent* Event = Collection->GetMutableDefinition().FindMutableEvent(EventHandle);
		if (!Event || !Event->GetActionCollection().IsValidIndex(DetailsView->ActionIndex))
		{
			return;
		}

		FSubsonicEventActionDefinition& ActionDef = Event->GetMutableActionCollection()[DetailsView->ActionIndex];
		FSubsonicEventActionBase* ActionBase = ActionDef.GetMutableActionBase();
		if (!ActionBase)
		{
			return;
		}

		const FName SourceParamName = SourceProperty->GetFName();

		TransactEventCollection(LOCTEXT("AddSubsonicBinding", "Add Subsonic Parameter Binding"), *Collection, [&](FSubsonicEventCollectionDefinition& CollectionDef)
		{
			ActionBase->AddPropertyBinding(SourceParamName, PropertyName);

			const UPropertyBag* CollectionBag = CollectionDef.GetParameters().GetPropertyBagStruct();
			const FInstancedPropertyBag* SourceBag = SourceStructIndex == CollectionContextIndex && CollectionBag != nullptr
				? &CollectionDef.GetParameters()
				: &Event->GetParameters();

			if (!SourceBag)
			{
				return;
			}

			// Re-resolve the source property from the current bag to avoid using a stale FProperty*
			// captured at menu creation time (the PropertyBag struct may have been rebuilt since then).
			const UPropertyBag* BagStruct = SourceBag->GetPropertyBagStruct();
			const FPropertyBagPropertyDesc* BagDesc = BagStruct ? BagStruct->FindPropertyDescByName(SourceParamName) : nullptr;
			if (!BagDesc || !BagDesc->CachedProperty)
			{
				return;
			}

			const FConstStructView BagView = SourceBag->GetValue();
			if (!BagView.IsValid())
			{
				return;
			}

			const UScriptStruct* ActionStruct = ActionDef.GetAction().GetScriptStruct();
			if (!ActionStruct)
			{
				return;
			}

			for (TFieldIterator<FProperty> PropIt(ActionStruct); PropIt; ++PropIt)
			{
				if (PropIt->GetFName() == PropertyName && BindingUtils::ArePropertiesBindingCompatible(BagDesc->CachedProperty , *PropIt))
				{
					const void* SrcPtr = BagDesc->CachedProperty->ContainerPtrToValuePtr<const void>(BagView.GetMemory());
					void* DstPtr = PropIt->ContainerPtrToValuePtr<void>(ActionBase);
					PropIt->CopyCompleteValue(DstPtr, SrcPtr);
					break;
				}
			}
		});
	}

	TSharedRef<FCheckBoxStyle> FSubsonicPropertyBindingExtension::GetCheckboxEntryStyle(const FProperty& BagProp)
	{
		FEdGraphPinType PinType;
		FLinearColor TypeColor = FLinearColor::White;
		if (const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>())
		{
			if (K2Schema->ConvertPropertyToPinType(&BagProp, PinType))
			{
				TypeColor = K2Schema->GetPinTypeColor(PinType);
			}
		}

		TSharedRef<FCheckBoxStyle> EntryStyle =  MakeShared<FCheckBoxStyle>(FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Menu.RadioButton"));
		EntryStyle->CheckedImage.TintColor = FSlateColor(TypeColor);
		EntryStyle->CheckedHoveredImage.TintColor = FSlateColor(TypeColor);
		EntryStyle->CheckedPressedImage.TintColor = FSlateColor(TypeColor);
		return EntryStyle;
	}
} // namespace UE::Subsonic::Editor

#undef LOCTEXT_NAMESPACE
