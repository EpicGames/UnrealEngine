// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyAccessEditor.h"
#include "PropertyBindingExtension.h"
#include "StructUtils/PropertyBag.h"


namespace UE::Subsonic
{
	// Forward Declarations
	class USubsonicEventCollection;

	namespace Core
	{
		struct FSubsonicEventCollectionDefinition;
	} // namespace Core

	namespace Editor
	{
		// Forward Declarations
		class USubsonicEventTreeDetailsView;

		// Extends FBindingContextStruct to accept a const UPropertyBag* and track
		// whether the binding originates from an event-level parameter bag.
		// The const_cast in the constructor is required because FBindingContextStruct::Struct
		// is a non-const UStruct* — the underlying property-access API is not const-correct.
		struct FSubsonicBindingContextStruct : public FBindingContextStruct
		{
			bool bIsEventBinding = false;

			FSubsonicBindingContextStruct(
				const UPropertyBag* InBag,
				const FSlateBrush* InIcon = nullptr,
				const FText& InDisplayText = FText::GetEmpty(),
				const FText& InTooltipText = FText::GetEmpty(),
				const FText& InSection = FText::GetEmpty())
				: FBindingContextStruct(const_cast<UPropertyBag*>(InBag), InIcon, InDisplayText, InTooltipText, InSection)
			{
			}
		};

		void TransactEventCollection(FText Description, USubsonicEventCollection& Collection, TFunctionRef<void(Core::FSubsonicEventCollectionDefinition&)> TransactionFunc);

		// Only shows the binding dropdown on properties not decorated with 'NoBinding' Metadata
		// tag. Overrides ExtendWidgetRow to populate the binding menu with parameters from the
		// collection- or event-level (Event value supersedes collection value if types match)
		// Subsonic 'Parameter' property bags.
		class FSubsonicPropertyBindingExtension : public FPropertyBindingExtension
		{
		public:
			using FNavigateToPropertyOwnerFunc = TFunction<void(FName)>;

			explicit FSubsonicPropertyBindingExtension(FNavigateToPropertyOwnerFunc InNavigateToPropertyOwner);

			//~ Begin IDetailPropertyExtensionHandler
			virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const override;
			virtual void ExtendWidgetRow(FDetailWidgetRow& WidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> InPropertyHandle) override;
			//~ End IDetailPropertyExtensionHandler

		private:
			// Index into the BindingContextStructs array built in GenerateBindingMenu / ExtendWidgetRow.
			// Collection is always added first (index 0), event second (index 1).
			static const int32 CollectionContextIndex = 0;

			FPropertyBindingWidgetArgs InitWidgetRow(
				USubsonicEventCollection& Collection,
				USubsonicEventTreeDetailsView& View,
				const FProperty* TargetProperty,
				FDetailWidgetRow& WidgetRow);

			TSharedRef<SWidget> GenerateBindingMenu(
				const FProperty* TargetProperty,
				TFunction<FName()> GetCurrentBindingParam,
				TWeakObjectPtr<USubsonicEventCollection> WeakCollection,
				TWeakObjectPtr<USubsonicEventTreeDetailsView> WeakView);

			static FText GetBindingText(
				TFunction<FName()> GetCurrentBindingParam,
				TWeakObjectPtr<USubsonicEventCollection> WeakCollection,
				TWeakObjectPtr<USubsonicEventTreeDetailsView> WeakView);

			static void RemoveBinding(
				TWeakObjectPtr<USubsonicEventCollection> WeakCollection,
				TWeakObjectPtr<USubsonicEventTreeDetailsView> WeakView,
				const FProperty* TargetProperty);

			static void AddParamBinding(
				USubsonicEventCollection* Collection,
				USubsonicEventTreeDetailsView* DetailsView,
				FName PropertyName,
				const int32 SourceStructIndex,
				const FProperty* SourceProperty);

			static TSharedRef<FCheckBoxStyle> GetCheckboxEntryStyle(const FProperty& BagProp);

			FNavigateToPropertyOwnerFunc NavigateToPropertyOwner;
		};
	} // namespace Editor
} // namespace UE::Subsonic
