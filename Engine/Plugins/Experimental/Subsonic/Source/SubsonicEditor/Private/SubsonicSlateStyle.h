// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioWidgetsStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"


namespace UE::Subsonic::Editor
{
	class FSlateStyle final : public FSlateStyleSet
	{
	public:
		FSlateStyle()
			: FSlateStyleSet("SubsonicStyle")
		{
			SetParentStyleName(FAudioWidgetsStyle::Get().GetStyleSetName());

			SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/Subsonic/Content/Editor/Slate"));
			SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

			static const FVector2D Icon20(20.0f, 20.0f);
			static const FVector2D Icon40(40.0f, 40.0f);
			static const FVector2D Icon64 = FVector2D(64.0f, 64.0f);

			// Subsonic Editor
			{
				Set("SubsonicEventCollection.Color", FColor(156, 0, 255));

				Set("SubsonicEditor.Arm.Small", new IMAGE_BRUSH_SVG(TEXT("Icons/arm"), Icon20));
				Set("SubsonicEditor.Play", new IMAGE_BRUSH_SVG(TEXT("Icons/play"), Icon40));
				Set("SubsonicEditor.Play.Small", new IMAGE_BRUSH_SVG(TEXT("Icons/play"), Icon20));
				Set("SubsonicEditor.Play.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/play_thumbnail"), Icon64));
				Set("SubsonicEditor.Play.Thumbnail.Hovered", new IMAGE_BRUSH_SVG(TEXT("Icons/play_thumbnail_hover"), Icon64));

				Set("SubsonicEditor.Play.Active.Valid", new IMAGE_BRUSH_SVG(TEXT("Icons/play_active_valid"), Icon40));
				Set("SubsonicEditor.Play.Active.Warning", new IMAGE_BRUSH_SVG(TEXT("Icons/play_active_warning"), Icon40));
				Set("SubsonicEditor.Play.Inactive.Valid", new IMAGE_BRUSH_SVG(TEXT("Icons/play_inactive_valid"), Icon40));
				Set("SubsonicEditor.Play.Inactive.Warning", new IMAGE_BRUSH_SVG(TEXT("Icons/play_inactive_warning"), Icon40));
				Set("SubsonicEditor.Play.Error", new IMAGE_BRUSH_SVG(TEXT("Icons/play_error"), Icon40));

				Set("SubsonicEditor.Stop", new IMAGE_BRUSH_SVG(TEXT("Icons/stop"), Icon40));

				Set("SubsonicEditor.Stop.Disabled", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_disabled"), Icon40));
				Set("SubsonicEditor.Stop.Active", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_active"), Icon40));
				Set("SubsonicEditor.Stop.Inactive", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_inactive"), Icon40));
				Set("SubsonicEditor.Stop.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_thumbnail"), Icon64));
				Set("SubsonicEditor.Stop.Thumbnail.Hovered", new IMAGE_BRUSH_SVG(TEXT("Icons/stop_thumbnail_hover"), Icon64));

				Set("SubsonicEditor.SubsonicEventCollection.Icon", new IMAGE_BRUSH_SVG(TEXT("Icons/subsoniceventcollection_icon"), Icon20));
				Set("SubsonicEditor.SubsonicEventCollection.Thumbnail", new IMAGE_BRUSH_SVG(TEXT("Icons/subsoniceventcollection_thumbnail"), Icon20));
			}

			FSlateStyleRegistry::RegisterSlateStyle(*this);
		}

		~FSlateStyle()
		{
			FSlateStyleRegistry::UnRegisterSlateStyle(*this);
		}
	};
} // namespace UE::Subsonic::Editor