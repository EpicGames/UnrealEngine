// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SubsonicEditor : ModuleRules
{
	public SubsonicEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"AudioEditor",
				"AudioMixer",
				"AudioWidgets",
				"BlueprintGraph",
				"ContentBrowser",
				"EditorSubsystem",
				"Engine",
				"GameplayTags",
				"GameplayTagsEditor",
				"InputCore",
				"PropertyBindingUtils",
				"PropertyBindingUtilsEditor",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"SubsonicCore",
				"SubsonicEngine",
				"ToolWidgets",
				"UnrealEd"
			}
		);

		bAllowUETypesInNamespaces = true;
	}
}
