// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandardActions/SubsonicAction_AudioComponent.h"

#include "Components/AudioComponent.h"
#include "IAudioModulation.h"
#include "Internationalization/Internationalization.h"
#include "StandardEventSubscribers/AudioComponentEventSubscriber.h"
#include "SubsonicDeviceUtils.h"
#include "SubsonicExecutor.h"
#include "SubsonicHandles.h"


#define LOCTEXT_NAMESPACE "SubsonicActions"

namespace UE::Subsonic
{
	namespace AudioComponentPrivate
	{
		UAudioComponent* AddScopedComponent(const Core::FExecutorScopeKey& InKey, ESubsonicExecutionScope Scope, FName Name)
		{
			UAudioComponent* Component = nullptr;
			if (USubsonicAudioComponentSubscriber* AudioComponentSubscriber = FindDeviceSubsystem<USubsonicAudioComponentSubscriber>(InKey))
			{
				switch (Scope)
				{
					case ESubsonicExecutionScope::Global:
					{
						Component = AudioComponentSubscriber->AddComponent(Name);
					}
					break;

					case ESubsonicExecutionScope::Executor:
					{
						Component = AudioComponentSubscriber->AddComponent(InKey, Name);
					}
					break;
				}
			}

			return Component;
		}

		UAudioComponent* FindScopedComponent(const Core::FExecutorScopeKey& InKey, ESubsonicExecutionScope Scope, FName Name)
		{
			UAudioComponent* Component = nullptr;
			if (USubsonicAudioComponentSubscriber* AudioComponentSubscriber = FindDeviceSubsystem<USubsonicAudioComponentSubscriber>(InKey))
			{
				switch (Scope)
				{
					case ESubsonicExecutionScope::Global:
					{
						Component = AudioComponentSubscriber->FindComponent(Name);
					}
					break;

					case ESubsonicExecutionScope::Executor:
					{
						Component = AudioComponentSubscriber->FindComponent(InKey, Name);
					}
					break;
				}
			}

			return Component;
		}

		UAudioComponent* AccessComponent(const Core::FExecutorScopeKey& InKey, ESubsonicAudioComponentAccess Access, ESubsonicExecutionScope Scope, FName Name)
		{
			UAudioComponent* Component = nullptr;
			if (Access == ESubsonicAudioComponentAccess::Find || Access == ESubsonicAudioComponentAccess::FindOrAdd)
			{
				Component = FindScopedComponent(InKey, Scope, Name);
			}

			bool bAddNewComponent = Access == ESubsonicAudioComponentAccess::Add;
			if (Access == ESubsonicAudioComponentAccess::FindOrAdd)
			{
				// Didn't find an existing component above, so attempt
				// to make one if set to find or add even if access not set to 'Add'.
				if (!Component)
				{
					bAddNewComponent = true;
				}
			}

			if (bAddNewComponent)
			{
				Component = AddScopedComponent(InKey, Scope, Name);
			}

			return Component;
		}
	} // namespace AudioComponentPrivate

	void FSubsonicEventAction_AudioComponentModify::Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const
	{
		const Core::FExecutorScopeKey ScopeKey(InExecutor);
		if (UAudioComponent* AudioComponent = AudioComponentPrivate::AccessComponent(ScopeKey, Access, Scope, Name))
		{
			for (const TInstancedStruct<FSubsonicEventAction_AudioComponentModifierBase>& Modifier : Modifiers)
			{
				if (const FSubsonicEventAction_AudioComponentModifierBase* ModPtr = Modifier.GetPtr())
				{
					ModPtr->Execute(InExecutor, *AudioComponent);
				}
			}
		}
	}

#if WITH_EDITOR
	FText FSubsonicEventAction_AudioComponentModify::GetDisplayInfo() const
	{
		FText NameText = FText::FromName(Name);
		if (NameText.IsEmptyOrWhitespace())
		{
			NameText = Super::GetDisplayInfo();
		}

		if (!Modifiers.IsEmpty())
		{
			if (const UScriptStruct* ModifierStruct = Modifiers[0].GetScriptStruct())
			{
				if (ModifierStruct&& Modifiers.Num() > 1)
				{
					return FText::Format(LOCTEXT("AudioComponentModifierListDisplaySinglePlusInfo_Format", "{0} ('{1}' and {2} more)"),
						NameText,
						ModifierStruct->GetDisplayNameText(),
						FText::AsNumber(Modifiers.Num() - 1));
				}

				return FText::Format(LOCTEXT("AudioComponentModifierListDisplayInfoSingle_Format", "{0} ({1})"),
					NameText,
					ModifierStruct->GetDisplayNameText());
			}

			return FText::Format(LOCTEXT("AudioComponentModifierListDisplayInfoNum_Format", "{0} ({1} modifiers)"),
				NameText,
				FText::AsNumber(Modifiers.Num()));
		}

		return NameText;
	}

	FText FSubsonicEventAction_AudioComponentPlay::GetDisplayInfo() const
	{
		if (!Name.IsNone())
		{
			if (Sound != nullptr)
			{
				return FText::Format(LOCTEXT("AudioComponentPlay_DisplayInfoFormat", "{0} ({1})"), FText::FromName(Name), FText::FromName(Sound->GetFName()));
			}

			return FText::FromName(Name);
		}

		return Super::GetDisplayInfo();
	}
#endif // WITH_EDITOR

	void FSubsonicEventAction_AudioComponentPlay::Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const
	{
		const Core::FExecutorScopeKey ScopeKey(InExecutor);
		if (UAudioComponent* AudioComponent = AudioComponentPrivate::AccessComponent(ScopeKey, Access, Scope, Name))
		{
			AudioComponent->SetSound(Sound);
			AudioComponent->Play();
		}
	}

	void FSubsonicEventAction_AudioComponentStop::Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const
	{
		const Core::FExecutorScopeKey ScopeKey(InExecutor);
		if (UAudioComponent* AudioComponent = AudioComponentPrivate::FindScopedComponent(ScopeKey, Scope, Name))
		{
			AudioComponent->Stop();
		}
	}

#if WITH_EDITOR
	FText FSubsonicEventAction_AudioComponentStop::GetDisplayInfo() const
	{
		if (!Name.IsNone())
		{
			return FText::FromName(Name);
		}

		return Super::GetDisplayInfo();
	}
#endif // WITH_EDITOR

	void FSubsonicEventAction_AudioComponentModifier_ExecuteOnFinished::Execute(const Core::FSubsonicExecutor& Executor, UAudioComponent& AudioComponent) const
	{
		AudioComponent.OnAudioFinishedNative.Remove(ActiveDelegate);
		ActiveDelegate = AudioComponent.OnAudioFinishedNative.AddLambda([ExecWeakPtr = Executor.AsWeak(), EventName = Event.GetTagName()](UAudioComponent* Component)
		{
			if (TSharedPtr<const Core::FSubsonicExecutor> ExecPtr = ExecWeakPtr.Pin())
			{
				ExecPtr->ExecuteEvent(EventName);
			}
		});
	}

	void FSubsonicEventAction_AudioComponentModifier_Play::Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const
	{
		AudioComponent.Play(StartTime);
	}

	void FSubsonicEventAction_AudioComponentModifier_SetAttenuation::Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const
	{
		AudioComponent.SetAttenuationSettings(Attenuation);
	}

	void FSubsonicEventAction_AudioComponentModifier_SetConcurrency::Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const
	{
		AudioComponent.ConcurrencySet = Concurrency;
	}

	void FSubsonicEventAction_AudioComponentModifier_SetModulationRouting::Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const
	{
		TSet<USoundModulatorBase*> ModulatorSet;
		Algo::Transform(Modulators, ModulatorSet, [](const TObjectPtr<USoundModulatorBase>& Modulator) { return Modulator.Get(); });
		AudioComponent.SetModulationRouting(ModulatorSet, Destination, RoutingMethod);
	}

	void FSubsonicEventAction_AudioComponentModifier_SetSound::Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const
	{
		AudioComponent.SetSound(Sound);
	}

	void FSubsonicEventAction_AudioComponentModifier_Stop::Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const
	{
		AudioComponent.Stop();
	}
} // namespace UE::Subsonic
#undef LOCTEXT_NAMESPACE // SubsonicActions
