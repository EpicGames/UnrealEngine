// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameplayTagContainer.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundModulationDestination.h"
#include "StructUtils/InstancedStruct.h"
#include "SubsonicEventCollection.h"
#include "SubsonicExecutor.h"

#include "SubsonicAction_AudioComponent.generated.h"


// Forward Declarations
class UAudioComponent;
class USoundConcurrency;

struct FSubsonicEventAction_AudioComponentModifierBase;


namespace UE::Subsonic
{
	UENUM()
	enum class ESubsonicAudioComponentAccess : uint8
	{
		// Adds new component, releasing any existing component.
		// Stops existing component if sound is not one-shot.
		Add,

		// Finds existing component, or creates a new one.
		FindOrAdd,

		// Ignore request and if component doesn't already exist.
		Find
	};

	USTRUCT(MinimalAPI, DisplayName = "Modify Audio Component")
	struct FSubsonicEventAction_AudioComponentModify : public Core::FSubsonicEventActionBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_AudioComponentModify() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const override;

#if WITH_EDITOR
		virtual FText GetDisplayInfo() const override;
#endif // WITH_EDITOR

		// Addressable name of audio component instance.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		FName Name;

		// Scope of component accessor.  If set to executor, named AudioComponent is
		// accessible only by pool of sibling event actions for a given
		// executor instance. If set to global, accesses any named AudioComponent
		// in the global pool.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		ESubsonicExecutionScope Scope = ESubsonicExecutionScope::Executor;

		UPROPERTY(EditAnywhere, Category = "Subsonic")
		ESubsonicAudioComponentAccess Access = ESubsonicAudioComponentAccess::FindOrAdd;

		UPROPERTY(EditAnywhere, Category = "Subsonic")
		TArray<TInstancedStruct<FSubsonicEventAction_AudioComponentModifierBase>> Modifiers;
	};

	USTRUCT(MinimalAPI, DisplayName = "Play Sound")
	struct FSubsonicEventAction_AudioComponentPlay : public Core::FSubsonicEventActionBase
	{
		GENERATED_BODY()

		virtual ~FSubsonicEventAction_AudioComponentPlay() = default;

#if WITH_EDITOR
		virtual FText GetDisplayInfo() const override;
#endif // WITH_EDITOR

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const override;

	public:
		// Addressable name of audio component instance.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		FName Name;

		// Sound to play.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		TObjectPtr<USoundBase> Sound;

		// Scope of component accessor.  If set to executor, named AudioComponent is
		// accessible only by pool of sibling event actions for a given
		// executor instance. If set to global, accesses any named AudioComponent
		// in the global pool.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		ESubsonicExecutionScope Scope = ESubsonicExecutionScope::Executor;

		UPROPERTY(EditAnywhere, Category = "Subsonic")
		ESubsonicAudioComponentAccess Access = ESubsonicAudioComponentAccess::FindOrAdd;
	};

	USTRUCT(MinimalAPI, DisplayName = "Stop Sound")
	struct FSubsonicEventAction_AudioComponentStop : public Core::FSubsonicEventActionBase
	{
		GENERATED_BODY()

		virtual ~FSubsonicEventAction_AudioComponentStop() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const override;

#if WITH_EDITOR
		virtual FText GetDisplayInfo() const override;
#endif // WITH_EDITOR

		// Addressable name of component/sound instance.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		FName Name;

		// Scope of component accessor.  If set to executor, named AudioComponent is
		// accessible only by pool of sibling event actions for a given
		// executor instance. If set to global, accesses any named AudioComponent
		// in the global pool.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		ESubsonicExecutionScope Scope = ESubsonicExecutionScope::Executor;
	};

	USTRUCT(MinimalAPI, meta = (Abstract, Hidden))
	struct FSubsonicEventAction_AudioComponentModifierBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_AudioComponentModifierBase() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const { }
	};

	// Executes event with the given name within the parent collection on completion
	USTRUCT(MinimalAPI, DisplayName = "Execute On Finished")
	struct FSubsonicEventAction_AudioComponentModifier_ExecuteOnFinished : public FSubsonicEventAction_AudioComponentModifierBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_AudioComponentModifier_ExecuteOnFinished() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const override;

		// Event in parent collection to execute.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		FGameplayTag Event;

	private:
		// Should really be tracked by subscriber to support cancelation, replacement, data-oriented execution ordering, const correctness, etc.
		mutable FDelegateHandle ActiveDelegate;
	};

	USTRUCT(MinimalAPI, DisplayName = "Play")
	struct FSubsonicEventAction_AudioComponentModifier_Play : public FSubsonicEventAction_AudioComponentModifierBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_AudioComponentModifier_Play() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const override;

		// Play StartTime Offset (plays immediately if 0.0)
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		float StartTime = 0.0f;
	};

	USTRUCT(MinimalAPI, DisplayName = "Set Attenuation")
	struct FSubsonicEventAction_AudioComponentModifier_SetAttenuation : public FSubsonicEventAction_AudioComponentModifierBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_AudioComponentModifier_SetAttenuation() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const override;

		UPROPERTY(EditAnywhere, Category = "Subsonic")
		TObjectPtr<USoundAttenuation> Attenuation;
	};

	USTRUCT(MinimalAPI, DisplayName = "Set Concurrency")
	struct FSubsonicEventAction_AudioComponentModifier_SetConcurrency : public FSubsonicEventAction_AudioComponentModifierBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_AudioComponentModifier_SetConcurrency() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const override;

		UPROPERTY(EditAnywhere, Category = "Subsonic")
		TSet<TObjectPtr<USoundConcurrency>> Concurrency;
	};

	USTRUCT(MinimalAPI, DisplayName = "Set Modulation Routing")
	struct FSubsonicEventAction_AudioComponentModifier_SetModulationRouting: public FSubsonicEventAction_AudioComponentModifierBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_AudioComponentModifier_SetModulationRouting() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const override;

		UPROPERTY(EditAnywhere, Category = "Subsonic")
		TSet<TObjectPtr<USoundModulatorBase>> Modulators;

		UPROPERTY(EditAnywhere, Category = "Subsonic")
		EModulationDestination Destination = EModulationDestination::Volume;

		UPROPERTY(EditAnywhere, Category = "Subsonic")
		EModulationRouting RoutingMethod = EModulationRouting::Inherit;
	};

	USTRUCT(MinimalAPI, DisplayName = "Set Sound")
	struct FSubsonicEventAction_AudioComponentModifier_SetSound : public FSubsonicEventAction_AudioComponentModifierBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_AudioComponentModifier_SetSound() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const override;

		// Sound to play.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		TObjectPtr<USoundBase> Sound;
	};

	USTRUCT(MinimalAPI, DisplayName = "Stop")
	struct FSubsonicEventAction_AudioComponentModifier_Stop : public FSubsonicEventAction_AudioComponentModifierBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_AudioComponentModifier_Stop() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, UAudioComponent& AudioComponent) const override;
	};
} // namespace UE::Subsonic
