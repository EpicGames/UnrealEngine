// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandardActions/SubsonicAction_EventCore.h"

#include "SubsonicExecutor.h"


namespace UE::Subsonic
{
	void FSubsonicEventAction_DelayEvent::Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const
	{
		if (EventName.IsValid())
		{
			if (Delay > 0.0f)
			{
				InExecutor.ExecuteEvent(EventName.GetTagName());
			}
			else
			{
				// TODO: Implement delay instance state tracking in subscriber & cycle detection
				// Detection likely should be implemented in validation step required to execute
				// in non-runtime contexts for perf reasons.
				InExecutor.ExecuteEvent(EventName.GetTagName());
			}
		}
	}
} // namespace UE::Subsonic