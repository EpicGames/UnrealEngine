// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameplayTagContainer.h"
#include "SubsonicEventCollection.h"

#include "SubsonicAction_EventCore.generated.h"


namespace UE::Subsonic
{
	namespace Core
	{
		// Forward Declarations
		struct FSubsonicExecutor;
	} // namespace Core

	UENUM()
	enum class EEventResolutionRule : uint8
	{
		First,
		Last,
		Random,
		Index
	};

	USTRUCT(MinimalAPI, BlueprintType, DisplayName = "Delay Event")
	struct FSubsonicEventAction_DelayEvent : public Core::FSubsonicEventActionBase
	{
		GENERATED_BODY()

	public:
		virtual ~FSubsonicEventAction_DelayEvent() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const override;

		// Identifier used to cancel or replace delay in progress.
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Action")
		FName DelayName = "Delay01";

		// Event to execute after the given delay.
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Action")
		FGameplayTag EventName;

		// Executes the given event by the prescribed amount.
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Action")
		float Delay = 1.0f;
	};
} // namespace UE::Subsonic