// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandardActions/SubsonicAction_GeneratorSource.h"

#include "IAudioMixerGeneratorSource.h"
#include "StandardEventSubscribers/SubsonicGeneratorSourceSubscriber.h"
#include "SubsonicDeviceUtils.h"
#include "SubsonicExecutor.h"
#include "SubsonicHandles.h"
#include "SubsonicParameterStore.h"

namespace UE::Subsonic
{
	void FSubsonicEventAction_GeneratorSourcePlay::Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const
	{
		using namespace UE::Subsonic::Core;

		if (Sound == nullptr)
		{
			return;
		}

		// Merge authored params (lowest priority) with trigger-time params from the executor.
		FSubsonicParameterStore MergedParams = Parameters;
		MergedParams.MergeFrom(InExecutor.GetParameters());

		const FExecutorScopeKey ScopeKey(InExecutor);
		if (USubsonicGeneratorSourceSubscriber* GeneratorSourceSubscriber = FindDeviceSubsystem<USubsonicGeneratorSourceSubscriber>(ScopeKey))
		{
			switch (Scope)
			{
			case ESubsonicExecutionScope::Global:
				GeneratorSourceSubscriber->PlaySound(Name, *Sound, MergedParams);
				break;

			case ESubsonicExecutionScope::Executor:
				GeneratorSourceSubscriber->PlaySound(ScopeKey, Name, *Sound, MergedParams);
				break;

			default:
				checkNoEntry();
			}
		}
	}

#if WITH_EDITOR
	FText FSubsonicEventAction_GeneratorSourcePlay::GetDisplayInfo() const
	{
		if (!Name.IsNone())
		{
			return FText::FromName(Name);
		}

		return Super::GetDisplayInfo();
	}
#endif // WITH_EDITOR

	void FSubsonicEventAction_GeneratorSourceStop::Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const
	{
		using namespace UE::Subsonic::Core;

		const FExecutorScopeKey ScopeKey(InExecutor);
		if (USubsonicGeneratorSourceSubscriber* GeneratorSourceSubscriber = FindDeviceSubsystem<USubsonicGeneratorSourceSubscriber>(ScopeKey))
		{
			switch (Scope)
			{
			case ESubsonicExecutionScope::Global:
				GeneratorSourceSubscriber->StopSound(Name);
				break;

			case ESubsonicExecutionScope::Executor:
				GeneratorSourceSubscriber->StopSound(ScopeKey, Name);
				break;

			default:
				checkNoEntry();
			}
		}
	}

#if WITH_EDITOR
	FText FSubsonicEventAction_GeneratorSourceStop::GetDisplayInfo() const
	{
		if (!Name.IsNone())
		{
			return FText::FromName(Name);
		}

		return Super::GetDisplayInfo();
	}
#endif // WITH_EDITOR
} // namespace UE::Subsonic
