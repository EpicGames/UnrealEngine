// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandardEventSubscribers/AudioComponentEventSubscriber.h"

#include "Sound/SoundBase.h"
#include "SubsonicCoreLog.h"
#include "SubsonicExecutor.h"
#include "SubsonicSubscriberDataStore.h"
#include "UObject/GCObject.h"
#include "UObject/UObjectGlobals.h"


namespace UE::Subsonic
{
	namespace AudioComponentEventSubscriberPrivate
	{
		struct FInstanceAudioComponents : public FGCObject
		{
		private:
			// Currently addressable component via handle
			TObjectPtr<UAudioComponent> ActiveComponent;
			FDelegateHandle OnFinishHandle;

			// Components no longer addressable from actions but actively
			// playing. Continually tracked here to avoid GC while finishing.
			TArray<TObjectPtr<UAudioComponent>> ReleasedComponents;
			TArray<FDelegateHandle> OnReleaseOnFinishHandles;

		public:
			const TObjectPtr<UAudioComponent> GetActiveComponent() const
			{
				return ActiveComponent;
			}

			const TArray<TObjectPtr<UAudioComponent>> GetReleasedComponents() const
			{
				return ReleasedComponents;
			}

			void SetActiveComponent(TObjectPtr<UAudioComponent> Component, FDelegateHandle&& InOnFinishHandle)
			{
				ActiveComponent = Component;
				OnFinishHandle = MoveTemp(InOnFinishHandle);
			}

			virtual void AddReferencedObjects(FReferenceCollector& Collector) override
			{
				Collector.AddReferencedObject(ActiveComponent);
				Collector.AddStableReferenceArray(&ReleasedComponents);
			}

			virtual FString GetReferencerName() const override
			{
				return "UE::Subsonic::InstanceAudioComponents";
			}

			void OnComponentFinished(UAudioComponent* FinishingComponent)
			{
				if (ActiveComponent.Get() == FinishingComponent)
				{
					ActiveComponent.Get()->OnAudioFinishedNative.Remove(OnFinishHandle);
					ActiveComponent = nullptr;
					OnFinishHandle.Reset();
				}

				for (int32 Index = ReleasedComponents.Num() - 1; Index >= 0; --Index)
				{
					if (ReleasedComponents[Index].Get() == FinishingComponent)
					{
						ReleasedComponents[Index]->OnAudioFinishedNative.Remove(OnReleaseOnFinishHandles[Index]);
						ReleasedComponents.RemoveAtSwap(Index);
						OnReleaseOnFinishHandles.RemoveAtSwap(Index);
					}
				}

			}

			bool IsEmpty() const
			{
				return !ActiveComponent.Get() && ReleasedComponents.IsEmpty();
			}

			void ReleaseComponent()
			{
				if (ActiveComponent.Get() && ActiveComponent->IsPlaying())
				{
					ReleasedComponents.Add(ActiveComponent);
					OnReleaseOnFinishHandles.Add(OnFinishHandle);
				}
				OnFinishHandle.Reset();
				ActiveComponent = nullptr;
			}

			void StopAll()
			{
				if (UAudioComponent* Component = ActiveComponent.Get())
				{
					Component->Stop();
					Component->OnAudioFinishedNative.Remove(OnFinishHandle);
				}

				for (int32 Index = 0; Index < ReleasedComponents.Num(); ++Index)
				{
					if (UAudioComponent* Component = ReleasedComponents[Index].Get())
					{
						Component->Stop();
						Component->OnAudioFinishedNative.Remove(OnReleaseOnFinishHandles[Index]);
					}
				}

				ActiveComponent = nullptr;
				OnFinishHandle.Reset();

				ReleasedComponents.Reset();
				OnReleaseOnFinishHandles.Reset();

			}
		};

		using FInstanceAudioComponentsPtr = TUniquePtr<FInstanceAudioComponents>;
		using FAudioComponentsCache = TMap<FName, FInstanceAudioComponentsPtr>;

		class FSubscriberImpl : public TSharedFromThis<FSubscriberImpl>
		{
		public:
			Core::TSubscriberDataStore<FInstanceAudioComponentsPtr> ComponentsStore;

			static UAudioComponent* CreateComponent(UObject& Parent, Audio::FDeviceId DeviceId, FName Name)
			{
				const FName ComponentName = MakeUniqueObjectName(&Parent, UAudioComponent::StaticClass(), Name);
				UAudioComponent* NewComponent = NewObject<UAudioComponent>(&Parent, ComponentName);

				check(NewComponent);
				NewComponent->AudioDeviceID = DeviceId;
				NewComponent->bAutoDestroy = true;
				NewComponent->bIsUISound = true;
				NewComponent->bAllowSpatialization = false;
				NewComponent->bReverb = false;
				NewComponent->bCenterChannelOnly = false;

				return NewComponent;
			}

			template <typename KeyType>
			bool ContainsCacheComponent(const TMap<KeyType, FAudioComponentsCache>& ScopeComponentMap, const KeyType& InKey, FName Name) const
			{
				if (const FAudioComponentsCache* Cache = ScopeComponentMap.Find(InKey))
				{
					if (const FInstanceAudioComponentsPtr* CacheComponentsPtr = Cache->Find(Name))
					{
						return CacheComponentsPtr->IsValid();
					}
				}

				return false;
			}

		public:
			void Deinitialize()
			{
				auto StopAll = [](FName, FInstanceAudioComponentsPtr& InstanceComponents)
				{
					if (InstanceComponents.IsValid())
					{
						InstanceComponents->StopAll();
					}
				};
				ComponentsStore.Empty(StopAll);
			}

			UAudioComponent* AddComponent(UObject& Parent, Audio::FDeviceId DeviceId, FName Name, bool bReleaseExisting)
			{
				FInstanceAudioComponentsPtr& InstanceComponents = ComponentsStore.FindOrAdd(Name);
				if (!InstanceComponents.IsValid())
				{
					InstanceComponents = MakeUnique<FInstanceAudioComponents>();
				}

				if (bReleaseExisting)
				{
					UAudioComponent* Component = InstanceComponents->GetActiveComponent();
					if (Component && Component->IsPlaying())
					{
						if (USoundBase* Sound = Component->GetSound(); Sound && !Sound->IsOneShot())
						{
							UE_LOGF(LogSubsonic, Warning,
								"Stopping released global sound with name '%ls': "
									"Sound is not one shot and continued play would result in leaked playback.",
								*Name.ToString())
							Component->Stop();
						}
					}

					InstanceComponents->ReleaseComponent();
				}

				TObjectPtr<UAudioComponent> ActiveComponent = InstanceComponents->GetActiveComponent();
				if (!ActiveComponent)
				{
					ActiveComponent = CreateComponent(Parent, DeviceId, Name);
					FDelegateHandle OnFinishedHandle = ActiveComponent->OnAudioFinishedNative.AddLambda([WeakImpl = AsWeak(), Name](UAudioComponent* FinishingComponent)
					{
						if (TSharedPtr<FSubscriberImpl> Impl = WeakImpl.Pin())
						{
							if (FInstanceAudioComponentsPtr* ComponentsPtr = Impl->ComponentsStore.Find(Name))
							{
								(*ComponentsPtr)->OnComponentFinished(FinishingComponent);
								if ((*ComponentsPtr)->IsEmpty())
								{
									Impl->ComponentsStore.Remove(Name);
								}
							}
						}
					});
					InstanceComponents->SetActiveComponent(ActiveComponent, MoveTemp(OnFinishedHandle));
				}

				return ActiveComponent;
			}

			UAudioComponent* AddComponent(UObject& Parent, Audio::FDeviceId DeviceId, const Core::FExecutorScopeKey& InKey, FName Name, bool bReleaseExisting)
			{
				FInstanceAudioComponentsPtr& InstanceComponents = ComponentsStore.FindOrAdd(InKey, Name);
				if (InstanceComponents.IsValid())
				{
					if (bReleaseExisting)
					{
						UAudioComponent* Component = InstanceComponents->GetActiveComponent();
						if (Component && Component->IsPlaying())
						{
							if (USoundBase* Sound = Component->GetSound(); Sound && !Sound->IsOneShot())
							{
								UE_LOGF(LogSubsonic, Warning,
									"Stopping released sound with handle '%ls': "
										"Sound is not one shot and continued play would result in leaked playback.",
									*InKey.ToString())
								Component->Stop();
							}
						}

						InstanceComponents->ReleaseComponent();
					}
					else
					{

					}
				}
				else
				{
					InstanceComponents = MakeUnique<FInstanceAudioComponents>();
				}

				TObjectPtr<UAudioComponent> ActiveComponent = InstanceComponents->GetActiveComponent();
				if (!ActiveComponent)
				{
					ActiveComponent = CreateComponent(Parent, DeviceId, Name);
					FDelegateHandle OnFinishedHandle = ActiveComponent->OnAudioFinishedNative.AddLambda([WeakImpl = AsWeak(), FinishKey = InKey, Name](UAudioComponent* FinishingComponent)
					{
						if (TSharedPtr<FSubscriberImpl> Impl = WeakImpl.Pin())
						{
							if (FInstanceAudioComponentsPtr* ComponentsPtr = Impl->ComponentsStore.Find(FinishKey, Name))
							{
								(*ComponentsPtr)->OnComponentFinished(FinishingComponent);
								if ((*ComponentsPtr)->IsEmpty())
								{
									Impl->ComponentsStore.Remove(FinishKey, Name);
								}
							}
						}
					});
					InstanceComponents->SetActiveComponent(ActiveComponent, MoveTemp(OnFinishedHandle));
				}

				return ActiveComponent;
			}

			UAudioComponent* FindComponent(FName Name) const
			{
				const FInstanceAudioComponentsPtr* InstanceComponents = ComponentsStore.Find(Name);
				if (InstanceComponents && InstanceComponents->IsValid())
				{
					return (*InstanceComponents)->GetActiveComponent();
				}

				return nullptr;
			}

			UAudioComponent* FindComponent(const Core::FExecutorScopeKey& InKey, FName Name) const
			{
				if (const FInstanceAudioComponentsPtr* ComponentsPtr = ComponentsStore.Find(InKey, Name))
				{
					return (*ComponentsPtr)->GetActiveComponent();
				}

				return nullptr;
			}

			void OnExecutorRegistered(const Core::FSubsonicExecutor& InExecutor)
			{
			}

			void OnExecutorUnregistered(const Core::FSubsonicExecutor& InExecutor)
			{
				const Core::FExecutorScopeKey Key(InExecutor);
				ComponentsStore.ForEach(Key, [this](FName, FInstanceAudioComponentsPtr& ComponentsPtr)
				{
					check(ComponentsPtr.IsValid());
					ComponentsPtr->StopAll();
				});
				ComponentsStore.Remove(Key);
			}

			void RemoveComponent(FName Name)
			{
				if (FInstanceAudioComponentsPtr* ComponentsPtr = ComponentsStore.Find(Name))
				{
					check(ComponentsPtr->IsValid());
					(*ComponentsPtr)->StopAll();
					ComponentsStore.Remove(Name);
				}
			}

			void RemoveComponent(const Core::FExecutorScopeKey& InKey, FName Name)
			{
				if (FInstanceAudioComponentsPtr* ComponentsPtr = ComponentsStore.Find(InKey, Name))
				{
					check(ComponentsPtr->IsValid());
					(*ComponentsPtr)->StopAll();
					ComponentsStore.Remove(InKey, Name);
				}
			}
		};
	} // namespace AudioComponentEventSubscriberPrivate

	void USubsonicAudioComponentSubscriber::Initialize(FSubsystemCollectionBase& Collection)
	{
		Super::Initialize(Collection);
		Impl = MakeShared<AudioComponentEventSubscriberPrivate::FSubscriberImpl>();
	}

	void USubsonicAudioComponentSubscriber::Deinitialize()
	{
		Impl->Deinitialize();
		Super::Deinitialize();
	}

	UAudioComponent* USubsonicAudioComponentSubscriber::AddComponent(FName Name, bool bReleaseExisting)
	{
		return Impl->AddComponent(*this, GetAudioDeviceId(), Name, bReleaseExisting);
	}

	UAudioComponent* USubsonicAudioComponentSubscriber::AddComponent(const Core::FExecutorScopeKey& InKey, FName Name, bool bReleaseExisting)
	{
		return Impl->AddComponent(*this, GetAudioDeviceId(), InKey, Name, bReleaseExisting);
	}

	UAudioComponent* USubsonicAudioComponentSubscriber::FindComponent(FName Name)
	{
		return Impl->FindComponent(Name);
	}

	UAudioComponent* USubsonicAudioComponentSubscriber::FindComponent(const Core::FExecutorScopeKey& InKey, FName Name)
	{
		return Impl->FindComponent(InKey, Name);
	}

	void USubsonicAudioComponentSubscriber::RemoveComponent(FName Name)
	{
		return Impl->RemoveComponent(Name);
	}

	void USubsonicAudioComponentSubscriber::RemoveComponent(const Core::FExecutorScopeKey& InKey, FName Name)
	{
		return Impl->RemoveComponent(InKey, Name);
	}

	void USubsonicAudioComponentSubscriber::OnExecutorRegistered(const Core::FSubsonicExecutor& InExecutor)
	{
		Impl->OnExecutorRegistered(InExecutor);
	}

	void USubsonicAudioComponentSubscriber::OnExecutorUnregistered(const Core::FSubsonicExecutor& InExecutor)
	{
		Impl->OnExecutorUnregistered(InExecutor);
	}
} // namespace UE::Subsonic
