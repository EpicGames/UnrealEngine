// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Subsystems/EngineSubsystem.h"

#include "Components/AudioComponent.h"
#include "Containers/Map.h"
#include "StandardEventSubscribers/SubsonicEventSubscriberBase.h"
#include "SubsonicEventSubscriberInterface.h"
#include "SubsonicExecutor.h"
#include "Subsystems/AudioEngineSubsystem.h"
#include "Templates/SharedPointer.h"

#include "AudioComponentEventSubscriber.generated.h"

#define UE_API SUBSONICENGINE_API


// Forward Declarations
namespace UE::Subsonic
{
	namespace AudioComponentEventSubscriberPrivate
	{
		class FSubscriberImpl;
	} // namespace AudioComponentEventSubscriberPrivate

	/**
	 * Subsonic Subscriber managing AudioComponent instances, events, and related data.
	 */
	UCLASS(MinimalAPI)
	class USubsonicAudioComponentSubscriber final : public USubsonicEventSubscriberBase
	{
		GENERATED_BODY()

	public:
		//~ Begin USubsonicEventSubscriberBase interface
		UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
		UE_API virtual void Deinitialize() override;
		//~ End USubsonicEventSubscriberBase interface

		UE_INTERNAL virtual void OnExecutorRegistered(const Core::FSubsonicExecutor& InExecutor) override;
		UE_INTERNAL virtual void OnExecutorUnregistered(const Core::FSubsonicExecutor& InExecutor) override;

		UE_INTERNAL UAudioComponent* AddComponent(FName Name, bool bReleaseExisting = true);
		UE_INTERNAL UAudioComponent* AddComponent(const Core::FExecutorScopeKey& InKey, FName Name, bool bReleaseExisting = true);

		UE_INTERNAL UAudioComponent* FindComponent(FName Name);
		UE_INTERNAL UAudioComponent* FindComponent(const Core::FExecutorScopeKey& InKey, FName Name);

		UE_INTERNAL void RemoveComponent(FName Name);
		UE_INTERNAL void RemoveComponent(const Core::FExecutorScopeKey& InKey, FName Name);

	private:
		TSharedPtr<AudioComponentEventSubscriberPrivate::FSubscriberImpl> Impl;
	};
} // namespace UE::Subsonic

#undef UE_API // SUBSONICENGINE_API
