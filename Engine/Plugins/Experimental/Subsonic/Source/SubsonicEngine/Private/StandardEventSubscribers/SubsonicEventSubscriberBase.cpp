// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandardEventSubscribers/SubsonicEventSubscriberBase.h"

namespace UE::Subsonic
{
	void USubsonicEventSubscriberBase::Initialize(FSubsystemCollectionBase& Collection)
	{
		Core::ISubsonicEventSubscriberInterface::Register();
	}

	void USubsonicEventSubscriberBase::Deinitialize()
	{
		Core::ISubsonicEventSubscriberInterface::Unregister();
	}
} // namespace UE::Subsonic