// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicGenerator.h"

#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"
#include "IAudioMixerGeneratorSource.h"
#include "StandardEventSubscribers/SubsonicWaveGenerator.h"
#include "StructUtils/PropertyBag.h"
#include "SubsonicBuiltInParameters.h"
#include "SubsonicCoreLog.h"
#include "SubsonicParameterStore.h"

namespace UE::Subsonic
{
	const TMap<FName, FSubsonicGenerator::FBuiltInParamHandler>& FSubsonicGenerator::GetBuiltInParams()
	{
		static const TMap<FName, FBuiltInParamHandler> Map = {
			{ BuiltInParameters::Volume,         &FSubsonicGenerator::SetVolume },
			{ BuiltInParameters::PitchShift,     &FSubsonicGenerator::SetPitchShift },
			{ BuiltInParameters::HighpassCutoff, &FSubsonicGenerator::SetHighpassCutoff },
			{ BuiltInParameters::LowpassCutoff,  &FSubsonicGenerator::SetLowpassCutoff },
			{ BuiltInParameters::FadeOutTime,    &FSubsonicGenerator::SetFadeOutTime },
		};
		return Map;
	}

	FSubsonicGenerator::FSubsonicGenerator(TSharedRef<FWaveGenerator> InWave, float InSampleRate)
		: InnerGenerator(InWave)
		, bGraphReady(true) // Wave path: no graph to wait for.
		, SampleRate(InSampleRate)
	{
		InitDSP();
	}

	FSubsonicGenerator::FSubsonicGenerator(ISoundGeneratorPtr InMetaSoundGenerator, float InSampleRate)
		: InnerGenerator(InMetaSoundGenerator)
		, bGraphReady(false) // Will be set true by the graph-set callback.
		, SampleRate(InSampleRate)
	{
		TSharedPtr<Metasound::FMetasoundGenerator, ESPMode::ThreadSafe> MSGen =
			StaticCastSharedPtr<Metasound::FMetasoundGenerator>(InMetaSoundGenerator);
		if (MSGen.IsValid())
		{
			MetaSoundGeneratorWeak = MSGen;

			// Register for notification when the MetaSound graph is compiled and ready.
			// If the graph is already built, AddGraphSetCallback fires immediately.
			GraphSetCallbackHandle = MSGen->AddGraphSetCallback(
				Metasound::FOnSetGraph::FDelegate::CreateLambda([this]()
				{
					bGraphReady.store(true, std::memory_order_release);
				}));
		}
		else
		{
			bGraphReady.store(true, std::memory_order_relaxed);
		}

		InitDSP();
	}

	FSubsonicGenerator::~FSubsonicGenerator()
	{
		if (GraphSetCallbackHandle.IsValid())
		{
			if (TSharedPtr<Metasound::FMetasoundGenerator, ESPMode::ThreadSafe> MSGen = MetaSoundGeneratorWeak.Pin())
			{
				MSGen->RemoveGraphSetCallback(GraphSetCallbackHandle);
			}
		}
	}

	void FSubsonicGenerator::InitDSP()
	{
		const int32 NumCh = InnerGenerator->GetNumChannels();

		HighpassFilter.Init(SampleRate, NumCh, Audio::EBiquadFilter::Highpass, 0.0f);
		HighpassFilter.SetEnabled(false);

		LowpassFilter.Init(SampleRate, NumCh, Audio::EBiquadFilter::Lowpass, SampleRate * 0.5f);
		LowpassFilter.SetEnabled(false);

		Resampler = MakeUnique<Audio::FRuntimeResampler>(NumCh);
		Resampler->SetFrameRatio(1.0f);
	}

	bool FSubsonicGenerator::TryGetFloat(const FSubsonicParameterStore& Store, const FPropertyBagPropertyDesc& Desc, float& OutValue)
	{
		if (Desc.ValueType == EPropertyBagPropertyType::Float)
		{
			TValueOrError<float, EPropertyBagResult> Result = Store.Bag.GetValueFloat(Desc);
			if (Result.HasValue())
			{
				OutValue = Result.GetValue();
				return true;
			}
		}
		else if (Desc.ValueType == EPropertyBagPropertyType::Double)
		{
			TValueOrError<double, EPropertyBagResult> Result = Store.Bag.GetValueDouble(Desc);
			if (Result.HasValue())
			{
				OutValue = static_cast<float>(Result.GetValue());
				return true;
			}
		}
		return false;
	}

	bool FSubsonicGenerator::HasMetaSoundInput(FName Name) const
	{
		TSharedPtr<Metasound::FMetasoundGenerator, ESPMode::ThreadSafe> MSGen = MetaSoundGeneratorWeak.Pin();
		if (!MSGen.IsValid())
		{
			return false;
		}
		return MSGen->GetInputWriteReference<float>(Name).IsSet();
	}

	void FSubsonicGenerator::ForwardToMetaSound(
		Metasound::FMetasoundGenerator& MSGen,
		const FSubsonicParameterStore& Store,
		const FPropertyBagPropertyDesc& Desc)
	{
		switch (Desc.ValueType)
		{
		case EPropertyBagPropertyType::Float:
		{
			TValueOrError<float, EPropertyBagResult> Result = Store.Bag.GetValueFloat(Desc);
			if (Result.HasValue())
			{
				MSGen.SetInputValue<float>(Desc.Name, Result.GetValue());
			}
			break;
		}
		case EPropertyBagPropertyType::Double:
		{
			TValueOrError<double, EPropertyBagResult> Result = Store.Bag.GetValueDouble(Desc);
			if (Result.HasValue())
			{
				MSGen.SetInputValue<float>(Desc.Name, static_cast<float>(Result.GetValue()));
			}
			break;
		}
		case EPropertyBagPropertyType::Bool:
		{
			TValueOrError<bool, EPropertyBagResult> Result = Store.Bag.GetValueBool(Desc);
			if (Result.HasValue())
			{
				MSGen.SetInputValue<bool>(Desc.Name, Result.GetValue());
			}
			break;
		}
		case EPropertyBagPropertyType::Int32:
		{
			TValueOrError<int32, EPropertyBagResult> Result = Store.Bag.GetValueInt32(Desc);
			if (Result.HasValue())
			{
				MSGen.SetInputValue<int32>(Desc.Name, Result.GetValue());
			}
			break;
		}
		default:
			break;
		}
	}

	void FSubsonicGenerator::ApplyParameters(const FSubsonicParameterStore& Store)
	{
		const UPropertyBag* BagStruct = Store.Bag.GetPropertyBagStruct();
		if (!BagStruct)
		{
			return;
		}

		// If the MetaSound graph isn't ready yet, defer the entire store.
		if (!bGraphReady.load(std::memory_order_acquire))
		{
			DeferredParams = MakeShared<FSubsonicParameterStore>(Store);
			return;
		}

		TSharedPtr<Metasound::FMetasoundGenerator, ESPMode::ThreadSafe> MSGen = MetaSoundGeneratorWeak.Pin();
		const TMap<FName, FBuiltInParamHandler>& BuiltInParams = GetBuiltInParams();

		for (const FPropertyBagPropertyDesc& Desc : BagStruct->GetPropertyDescs())
		{
			// Try to extract as float (covers float and double bag types).
			float FloatValue = 0.0f;
			if (!TryGetFloat(Store, Desc, FloatValue))
			{
				// Non-float type (bool, int32, etc.) -forward to MetaSound only.
				if (MSGen.IsValid())
				{
					ForwardToMetaSound(*MSGen, Store, Desc);
				}
				continue;
			}

			// If MetaSound has a matching input, send it there and skip the built-in handler.
			if (MSGen.IsValid() && HasMetaSoundInput(Desc.Name))
			{
				MSGen->SetInputValue<float>(Desc.Name, FloatValue);
				continue;
			}

			// Check if this is a built-in parameter with a dedicated DSP handler.
			if (const FBuiltInParamHandler* Handler = BuiltInParams.Find(Desc.Name))
			{
				(this->**Handler)(FloatValue);
				continue;
			}

			// Unknown float param with no MetaSound input -ignore.
		}
	}

	void FSubsonicGenerator::SetVolume(float dB)
	{
		VolumeLinear = Audio::ConvertToLinear(dB);
	}

	void FSubsonicGenerator::SetPitchShift(float Semitones)
	{
		constexpr int32 PitchShiftInterpolationFrames = 64;
		const float Ratio = FMath::Pow(2.0f, Semitones / 12.0f);
		Resampler->SetFrameRatio(Ratio, PitchShiftInterpolationFrames);
		if (!FMath::IsNearlyZero(Semitones))
		{
			bPitchShiftActive = true;
		}
	}

	void FSubsonicGenerator::SetHighpassCutoff(float FreqHz)
	{
		if (FreqHz <= 0.0f)
		{
			HighpassFilter.SetEnabled(false);
		}
		else
		{
			HighpassFilter.SetEnabled(true);
			HighpassFilter.SetFrequency(FreqHz);
		}
	}

	void FSubsonicGenerator::SetLowpassCutoff(float FreqHz)
	{
		if (FreqHz <= 0.0f || FreqHz >= SampleRate * 0.5f)
		{
			LowpassFilter.SetEnabled(false);
		}
		else
		{
			LowpassFilter.SetEnabled(true);
			LowpassFilter.SetFrequency(FreqHz);
		}
	}

	void FSubsonicGenerator::SetFadeOutTime(float Seconds)
	{
		if (FadeGainDelta == 0.0f)
		{
			FadeOutTimeSec = Seconds;
		}
	}

	void FSubsonicGenerator::SetSource(Audio::IAudioMixerGeneratorSource* InSource)
	{
		Source = InSource;
	}

	void FSubsonicGenerator::Play()
	{
		if (Source)
		{
			Source->Play();
		}
	}

	void FSubsonicGenerator::Stop()
	{
		if (FadeGainDelta != 0.0f)
		{
			return;
		}

		if (FadeOutTimeSec > 0.0f)
		{
			// Begin fade -OnGenerateAudio will null Source when fade completes.
			FadeGainDelta = 1.0f / (FadeOutTimeSec * SampleRate);
			return;
		}

		// Immediate stop.
		if (Source)
		{
			Source->Stop();
			Source = nullptr;
		}
	}

	Audio::IAudioMixerGeneratorSource* FSubsonicGenerator::GetSource() const
	{
		return Source;
	}

	int32 FSubsonicGenerator::OnGenerateAudio(float* OutAudio, int32 InNumSamples)
	{
		// Apply deferred params once graph becomes ready.
		if (DeferredParams && bGraphReady.load(std::memory_order_acquire))
		{
			ApplyParameters(*DeferredParams);
			DeferredParams.Reset();
		}

		const int32 NumChannels = InnerGenerator->GetNumChannels();
		const int32 NumFrames = (NumChannels > 0) ? (InNumSamples / NumChannels) : InNumSamples;

		int32 Result = 0;

		// Signal chain: InnerGenerator ->[Resample/PitchShift] ->[Highpass] ->[Lowpass] ->Volume ->Output
		if (bPitchShiftActive)
		{
			// Pitch shift active: generate more/fewer inner samples, then resample to output size.
			const int32 NumInputFrames = Resampler->GetNumInputFramesNeededToProduceOutputFrames(NumFrames);
			int32 NumInputSamples = NumInputFrames * NumChannels;

			const int32 LeftoverSamples = ResampleInputBuffer.Num();
			ResampleInputBuffer.SetNumUninitialized(FMath::Max(LeftoverSamples, NumInputSamples));
			if (NumInputSamples > LeftoverSamples)
			{
				NumInputSamples = LeftoverSamples + InnerGenerator->OnGenerateAudio(ResampleInputBuffer.GetData() + LeftoverSamples, NumInputSamples - LeftoverSamples);
			}
			else
			{
				NumInputSamples = LeftoverSamples;
			}

			int32 InputFramesConsumed = 0;
			int32 OutputFramesProduced = 0;
			Resampler->ProcessInterleaved(
				TArrayView<const float>(ResampleInputBuffer.GetData(), NumInputSamples),
				TArrayView<float>(OutAudio, InNumSamples),
				InputFramesConsumed,
				OutputFramesProduced);

			const int32 ProducedSamples = OutputFramesProduced * NumChannels;
			Result = ProducedSamples;

			// Save leftover frames for next run.
			const int32 InputSamplesConsumed = InputFramesConsumed * NumChannels;
			const int32 NewLeftoverSamples = NumInputSamples - InputSamplesConsumed;
			if (NewLeftoverSamples > 0)
			{
				FMemory::Memmove(ResampleInputBuffer.GetData(), ResampleInputBuffer.GetData() + InputSamplesConsumed, NewLeftoverSamples * sizeof(float));
			}
			ResampleInputBuffer.SetNum(NewLeftoverSamples);
		}
		else
		{
			Result = InnerGenerator->OnGenerateAudio(OutAudio, InNumSamples);
		}

		// Apply filters in-place (no-op when disabled).
		HighpassFilter.ProcessAudio(OutAudio, InNumSamples, OutAudio);
		LowpassFilter.ProcessAudio(OutAudio, InNumSamples, OutAudio);

		// Apply volume and fade-out in a single pass via SIMD-optimized ramp.
		{
			const float FadeStart = FadeGain;
			if (FadeGainDelta != 0.0f)
			{
				FadeGain = FMath::Max(0.0f, FadeGain - FadeGainDelta * NumFrames);
			}

			const float StartGain = VolumeLinear * FadeStart;
			const float EndGain = VolumeLinear * FadeGain;
			if (StartGain != 1.0f || EndGain != 1.0f)
			{
				Audio::ArrayFade(TArrayView<float>(OutAudio, InNumSamples), StartGain, EndGain);
			}
		}

		// When the generator is done (fade finished or inner generator ended),
		// stop the source and null the pointer so the subscriber knows to clean up.
		if (Source && IsFinished())
		{
			Source->Stop();
			Source = nullptr;
		}

		return Result;
	}

	int32 FSubsonicGenerator::GetNumChannels() const
	{
		return InnerGenerator->GetNumChannels();
	}

	bool FSubsonicGenerator::IsFinished() const
	{
		return FadeGain <= 0.0f || InnerGenerator->IsFinished();
	}

} // namespace UE::Subsonic
