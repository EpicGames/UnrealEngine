// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/Filter.h"
#include "DSP/RuntimeResampler.h"
#include "MetasoundGenerator.h"
#include "Sound/SoundGenerator.h"

#include <atomic>

namespace Audio { class IAudioMixerGeneratorSource; }

namespace UE::Subsonic
{
	class FWaveGenerator;
	struct FSubsonicParameterStore;

	// Wrapper ISoundGenerator that owns either a WaveGenerator or MetsoundGenerator inner generator.
	// Applies DSP (volume, pitch shift, highpass, lowpass) after delegating audio
	//  generation to the inner generator.
	//
	// After initialization, these generators are owned by the audio render thread and should be communicated
	//  with only through SubsonicRelay.
	class FSubsonicGenerator : public ISoundGenerator
	{
	public:
		// Wave path: inner generator is an FWaveGenerator (pure wave player).
		explicit FSubsonicGenerator(TSharedRef<FWaveGenerator> InWave, float InSampleRate);

		// MetaSound path: inner generator is an FMetasoundGenerator (returned by UMetaSoundSource).
		explicit FSubsonicGenerator(ISoundGeneratorPtr InMetaSoundGenerator, float InSampleRate);

		virtual ~FSubsonicGenerator() override;

		//~ Begin ISoundGenerator
		virtual int32 OnGenerateAudio(float* OutAudio, int32 InNumSamples) override;
		virtual int32 GetNumChannels() const override;
		virtual bool IsFinished() const override;
		//~ End ISoundGenerator

		// Audio render thread: apply named parameters from the store. Built-in parameters are
		// handled by dedicated DSP handlers, unless the MetasoundGenerator provides a matching input.
		void ApplyParameters(const FSubsonicParameterStore& Store);

		// Associate a GeneratorSource with this generator, so the generator can be in charge of source state
		//  (play, stop, pause, etc.)
		void SetSource(Audio::IAudioMixerGeneratorSource* InSource);

		// Start playback via the associated GeneratorSource.
		void Play();

		// Stop playback. If FadeOutTime > 0, begins a fade; otherwise calls Source->Stop() immediately.
		void Stop();

		// Game thread: returns the bare source pointer. Null means the generator is done
		//  and the subscriber can safely destroy the source.
		Audio::IAudioMixerGeneratorSource* GetSource() const;

	private:
		using FBuiltInParamHandler = void (FSubsonicGenerator::*)(float);
		static const TMap<FName, FBuiltInParamHandler>& GetBuiltInParams();

		// Attempts to extract a float from a property bag descriptor (handles float and double types).
		static bool TryGetFloat(const FSubsonicParameterStore& Store, const FPropertyBagPropertyDesc& Desc, float& OutValue);

		// Returns true if the pinned MetaSound generator has an input with the given name.
		bool HasMetaSoundInput(FName Name) const;

		// Forwards a non-float parameter to the pinned MetaSound generator.
		void ForwardToMetaSound(Metasound::FMetasoundGenerator& MSGen, const FSubsonicParameterStore& Store, const FPropertyBagPropertyDesc& Desc);

		// Built-in parameter handlers.
		void SetVolume(float dB);
		void SetPitchShift(float Semitones);
		void SetHighpassCutoff(float FreqHz);
		void SetLowpassCutoff(float FreqHz);
		void SetFadeOutTime(float Seconds);

		// Initializes DSP state (filters, resampler) after InnerGenerator is set.
		void InitDSP();

		ISoundGeneratorPtr InnerGenerator;

		// Non-null for MetaSound path only.
		TWeakPtr<Metasound::FMetasoundGenerator, ESPMode::ThreadSafe> MetaSoundGeneratorWeak;

		// True once the MetaSound graph has been built and its inputs are available.
		// Set via AddGraphSetCallback; always true for wave generators.
		std::atomic<bool> bGraphReady{ false };

		// Handle for the graph-set callback so we can unregister in the destructor.
		FDelegateHandle GraphSetCallbackHandle;

		// Holds the most recent SetParameters payload when the MetaSound graph is not yet ready.
		// Applied and cleared once bGraphReady becomes true. Last-writer-wins.
		TSharedPtr<FSubsonicParameterStore> DeferredParams;

		// Current volume as linear gain. Updated from the "Volume" parameter (interpreted as dB).
		// 0 dB = 1.0 linear (unity gain). Render thread only.
		float VolumeLinear = 1.0f;

		// Cached sample rate from the audio device.
		float SampleRate = 0.0f;

		// DSP processors applied in the signal chain after the inner generator.
		Audio::FBiquadFilter HighpassFilter;
		Audio::FBiquadFilter LowpassFilter;

		// Pitch shifting via resampling.
		TUniquePtr<Audio::FRuntimeResampler> Resampler;
		TArray<float> ResampleInputBuffer;
		bool bPitchShiftActive = false;

		// Fade-out envelope state. Render thread only.
		// FadeGainDelta > 0 means a fade is in progress; FadeGain <= 0 means it has completed.
		float FadeOutTimeSec = 0.0f;
		float FadeGain = 1.0f;
		float FadeGainDelta = 0.0f;

		// Non-owning pointer to the associated GeneratorSource. Set on the game thread
		//  at creation, nulled on the render thread when the generator is done.
		Audio::IAudioMixerGeneratorSource* Source = nullptr;
	};

} // namespace UE::Subsonic
