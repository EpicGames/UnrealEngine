// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandardEventSubscribers/SubsonicGeneratorSourceSubscriber.h"

#include "Algo/Transform.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "Engine/Engine.h"
#include "IAudioMixerGeneratorSource.h"
#include "MetasoundSource.h"
#include "Misc/App.h"
#include "StandardEventSubscribers/SubsonicRelay.h"
#include "SubsonicCoreLog.h"
#include "SubsonicGenerator.h"
#include "SubsonicSubsystem.h"
#include "SubsonicWaveGenerator.h"
#include "UObject/UObjectGlobals.h"


namespace UE::Subsonic
{
	void FInstanceSources::ApplyParameters(FSubsonicRelay& Relay)
	{
		if (!Generator)
		{
			return;
		}

		// Always allocate a fresh merged store - the audio thread owns its TSharedPtr.
		TSharedPtr<FSubsonicParameterStore> Merged = MakeShared<FSubsonicParameterStore>();
		Merged->MergeFrom(AuthoredParams);
		Merged->MergeFrom(RuntimeParams);

		// Enqueue for the audio thread via the relay's global command queue.
		FRelayCommand Cmd;
		Cmd.Type = FRelayCommand::EType::SetParameters;
		Cmd.Params = MoveTemp(Merged);
		Cmd.Target = Generator;
		Relay.EnqueueCommand(MoveTemp(Cmd));
	}

	namespace SubscriberPrivate
	{
		// Creates an FSubsonicGenerator from a USoundWave, using the MetaSound path
		// if the sound is a UMetaSoundSource, otherwise the wave path.
		TSharedPtr<FSubsonicGenerator> CreateGenerator(USoundWave& Sound, Audio::FMixerDevice& AudioDevice)
		{
			const float SampleRate = AudioDevice.GetSampleRate();

			if (UMetaSoundSource* MetaSound = Cast<UMetaSoundSource>(&Sound))
			{
				// Ensure the MetaSound graph is registered and compiled.
				MetaSound->InitResources();

				FSoundGeneratorInitParams InitParams;
				InitParams.AudioDeviceID = AudioDevice.DeviceID;
				InitParams.SampleRate = SampleRate;
				InitParams.AudioMixerNumOutputFrames = AudioDevice.GetNumOutputFrames();
				InitParams.NumChannels = MetaSound->NumChannels;
				InitParams.NumFramesPerCallback = AudioDevice.GetNumOutputFrames();
				InitParams.AudioComponentId = 0;
				InitParams.InstanceID = 0;
				InitParams.bIsPreviewSound = false;

				TArray<FAudioParameter> DefaultParams;
				MetaSound->InitParameters(DefaultParams, NAME_None);
				ISoundGeneratorPtr MSGenPtr = MetaSound->CreateSoundGenerator(InitParams, MoveTemp(DefaultParams));
				if (!MSGenPtr.IsValid())
				{
					return nullptr;
				}

				return MakeShared<FSubsonicGenerator>(MSGenPtr, SampleRate);
			}

			// Skip the wave path in non-audio contexts. USoundWave::GetSoundWaveProxy() returns
			// a default-initialized proxy when !FApp::CanEverRenderAudio(); the empty data
			// behind it isn't useful for playback. Mirrors the gate on Main.
			if (!FApp::CanEverRenderAudio())
			{
				return nullptr;
			}

			const TSharedRef<const FSoundWaveProxy> Proxy = Sound.GetSoundWaveProxy();
			TSharedRef<FWaveGenerator> WaveGen = MakeShared<FWaveGenerator>(Proxy->GetSoundWaveDataRef(), &AudioDevice);
			return MakeShared<FSubsonicGenerator>(WaveGen, SampleRate);
		}

		void PlaySoundOnInstance(
			USubsonicGeneratorSourceSubscriber& Subscriber,
			USoundWave& Sound,
			FInstanceSources& InstanceSources,
			FSubsonicRelay& Relay,
			const FSubsonicParameterStore& Params,
			bool bReleaseExisting)
		{
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				if (Audio::FMixerDevice* AudioDevice = Subscriber.GetMutableMixerDevice())
				{
					TSharedPtr<FSubsonicGenerator> Generator = CreateGenerator(Sound, *AudioDevice);
					if (!Generator.IsValid())
					{
						return;
					}

					TUniquePtr<Audio::IAudioMixerGeneratorSource> NewSource = AudioDevice->CreateGeneratorSource(Generator.ToSharedRef());
					if (NewSource.IsValid())
					{
						if (InstanceSources.ActiveSource != nullptr)
						{
							FReleasedSource Released;
							Released.Source = MoveTemp(InstanceSources.ActiveSource);
							Released.Generator = InstanceSources.Generator;
							InstanceSources.ReleasedSources.Add(MoveTemp(Released));
						}

						InstanceSources.Generator = Generator;
						InstanceSources.AuthoredParams = Params;
						InstanceSources.RuntimeParams.Reset();
						InstanceSources.ApplyParameters(Relay);

						Generator->SetSource(NewSource.Get());
						Relay.EnqueueCommand(FRelayCommand { FRelayCommand::EType::Play, nullptr, Generator });
						InstanceSources.ActiveSource = MoveTemp(NewSource);
					}
				}
			}
		}

		void StopSoundOnInstance(FInstanceSources& Instances, FSubsonicRelay& Relay)
		{
			if (Instances.ActiveSource != nullptr)
			{
				FReleasedSource Released;
				Released.Source = MoveTemp(Instances.ActiveSource);
				Released.Generator = Instances.Generator;
				Instances.ReleasedSources.Add(MoveTemp(Released));

				Relay.EnqueueCommand(FRelayCommand
				{
					FRelayCommand::EType::Stop, nullptr, Instances.Generator
				});

				Instances.Generator.Reset();
			}
		}

		void StopAllOnInstance(FInstanceSources& Instances, FSubsonicRelay& Relay)
		{
			if (Instances.Generator)
			{
				Relay.EnqueueCommand(FRelayCommand
				{
					FRelayCommand::EType::Stop, nullptr, Instances.Generator
				});
			}

			for (FReleasedSource& Released : Instances.ReleasedSources)
			{
				if (Released.Generator)
				{
					Relay.EnqueueCommand(FRelayCommand
					{
						FRelayCommand::EType::Stop, nullptr, Released.Generator
					});
				}
			}

			Instances.ActiveSource.Reset();
			Instances.Generator.Reset();
			Instances.ReleasedSources.Empty();
		}

		// Returns true if the instance is fully empty and can be removed from the store.
		bool ClearStaleSoundsOnInstance(FInstanceSources& Instances)
		{
			Instances.ReleasedSources.RemoveAll([](const FReleasedSource& Released)
			{
				return !Released.Generator || Released.Generator->GetSource() == nullptr;
			});

			if (Instances.ActiveSource != nullptr && Instances.Generator && Instances.Generator->GetSource() == nullptr)
			{
				Instances.ActiveSource.Reset();
				Instances.Generator.Reset();
			}

			return Instances.ActiveSource == nullptr && Instances.ReleasedSources.IsEmpty();
		}
	} // namespace SubscriberPrivate

	USubsonicGeneratorSourceSubscriber::USubsonicGeneratorSourceSubscriber()
		: FTickableGameObject(ETickableTickType::Never)
		, Relay(MakeShared<FSubsonicRelay>())
	{
	}

	void USubsonicGeneratorSourceSubscriber::Initialize(FSubsystemCollectionBase& Collection)
	{
		Super::Initialize(Collection);
		SetTickableTickType(ETickableTickType::Always);
	}

	void USubsonicGeneratorSourceSubscriber::Deinitialize()
	{
		SetTickableTickType(ETickableTickType::Never);

		auto StopAll = [this](FName, FInstanceSources& Instances)
		{
			SubscriberPrivate::StopAllOnInstance(Instances, *Relay);
		};
		SourcesDataStore.Empty(StopAll);

		Super::Deinitialize();
	}

	void USubsonicGeneratorSourceSubscriber::OnCollectionRegistered(const FCollectionHandle& InHandle)
	{
	}

	void USubsonicGeneratorSourceSubscriber::OnCollectionUnregistered(const FCollectionHandle& InHandle)
	{
	}

	void USubsonicGeneratorSourceSubscriber::OnExecutorRegistered(const Core::FSubsonicExecutor& InExecutor)
	{
	}

	void USubsonicGeneratorSourceSubscriber::OnExecutorUnregistered(const Core::FSubsonicExecutor& InExecutor)
	{
		const FExecutorScopeKey Key(InExecutor);
		SourcesDataStore.ForEach(Key, [this](FName, FInstanceSources& Instances)
		{
			SubscriberPrivate::StopAllOnInstance(Instances, *Relay);
		});
		SourcesDataStore.Remove(Key);
	}

	void USubsonicGeneratorSourceSubscriber::PlaySound(FName Name, USoundWave& Sound, const FSubsonicParameterStore& Params, bool bReleaseExisting)
	{
		FInstanceSources& Instances = SourcesDataStore.FindOrAdd(Name);
		SubscriberPrivate::PlaySoundOnInstance(*this, Sound, Instances, *Relay, Params, bReleaseExisting);
	}

	void USubsonicGeneratorSourceSubscriber::PlaySound(const FExecutorScopeKey& InKey, FName Name, USoundWave& Sound, const FSubsonicParameterStore& Params, bool bReleaseExisting)
	{
		FInstanceSources& Instances = SourcesDataStore.FindOrAdd(InKey, Name);
		SubscriberPrivate::PlaySoundOnInstance(*this, Sound, Instances, *Relay, Params, bReleaseExisting);
	}

	void USubsonicGeneratorSourceSubscriber::StopSound(FName Name)
	{
		if (FInstanceSources* Instances = SourcesDataStore.Find(Name))
		{
			SubscriberPrivate::StopSoundOnInstance(*Instances, *Relay);
		}
	}

	void USubsonicGeneratorSourceSubscriber::StopSound(const FExecutorScopeKey& InKey, FName Name)
	{
		if (FInstanceSources* Instances = SourcesDataStore.Find(InKey, Name))
		{
			SubscriberPrivate::StopSoundOnInstance(*Instances, *Relay);
		}
	}

	void USubsonicGeneratorSourceSubscriber::SetParameters(FName SourceName, const FSubsonicParameterStore& Params)
	{
		if (FInstanceSources* Instances = SourcesDataStore.Find(SourceName))
		{
			Instances->RuntimeParams.MergeFrom(Params);
			Instances->ApplyParameters(*Relay);
		}
	}

	void USubsonicGeneratorSourceSubscriber::SetParameters(const FExecutorScopeKey& InKey, FName SourceName, const FSubsonicParameterStore& Params)
	{
		if (FInstanceSources* Instances = SourcesDataStore.Find(InKey, SourceName))
		{
			Instances->RuntimeParams.MergeFrom(Params);
			Instances->ApplyParameters(*Relay);
		}
	}

	void USubsonicGeneratorSourceSubscriber::SetAuthoredParameters(FName SourceName, FSubsonicParameterStore Params)
	{
		if (FInstanceSources* Instances = SourcesDataStore.Find(SourceName))
		{
			Instances->AuthoredParams = MoveTemp(Params);
			Instances->ApplyParameters(*Relay);
		}
	}

	void USubsonicGeneratorSourceSubscriber::SetAuthoredParameters(const FExecutorScopeKey& InKey, FName SourceName, FSubsonicParameterStore Params)
	{
		if (FInstanceSources* Instances = SourcesDataStore.Find(InKey, SourceName))
		{
			Instances->AuthoredParams = MoveTemp(Params);
			Instances->ApplyParameters(*Relay);
		}
	}

	void USubsonicGeneratorSourceSubscriber::Tick(float DeltaTime)
	{
		if (Audio::FMixerDevice* MixerDevice = GetMutableMixerDevice())
		{
			Relay->Tick(*MixerDevice);
		}

		// Clear stale sounds at global scope.
		TArray<FName> GlobalNamesToClear;
		SourcesDataStore.ForEach([&GlobalNamesToClear](FName Name, FInstanceSources& Instances)
		{
			if (SubscriberPrivate::ClearStaleSoundsOnInstance(Instances))
			{
				GlobalNamesToClear.Add(Name);
			}
		});
		for (FName Name : GlobalNamesToClear)
		{
			SourcesDataStore.Remove(Name);
		}

		// Clear stale sounds at executor scope.
		TArray<TPair<FExecutorScopeKey, FName>> ToRemove;
		SourcesDataStore.ForEach([&ToRemove](const FExecutorScopeKey& Key, const FName& Name, FInstanceSources& Instances)
		{
			if (SubscriberPrivate::ClearStaleSoundsOnInstance(Instances))
			{
				ToRemove.Emplace(Key, Name);
			}
		});

		for (const TPair<FExecutorScopeKey, FName>& Entry : ToRemove)
		{
			SourcesDataStore.Remove(Entry.Key, Entry.Value);
		}
	}

	TStatId USubsonicGeneratorSourceSubscriber::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(USubsonicGeneratorSourceSubscriber, STATGROUP_Tickables);
	}
}
