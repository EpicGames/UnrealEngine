// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandardEventSubscribers/SubsonicRelay.h"

#include "AudioMixerDevice.h"
#include "SubsonicGenerator.h"

namespace UE::Subsonic
{
	void FSubsonicRelay::EnqueueCommand(FRelayCommand&& Cmd)
	{
		FScopeLock ScopeLock(&CommandLock);
		PendingCommands.Add(MoveTemp(Cmd));
	}

	void FSubsonicRelay::Tick(Audio::FMixerDevice& MixerDevice)
	{
		TArray<FRelayCommand> Commands;
		{
			FScopeLock ScopeLock(&CommandLock);
			Commands = MoveTemp(PendingCommands);
			// PendingCommands is now empty after move
		}

		if (Commands.IsEmpty())
		{
			return;
		}

		MixerDevice.AudioRenderThreadCommand([Cmds = MoveTemp(Commands)]()
		{
			for (const FRelayCommand& Cmd : Cmds)
			{
				if (!Cmd.Target)
				{
					continue;
				}

				switch (Cmd.Type)
				{
				case FRelayCommand::EType::SetParameters:
					if (Cmd.Params)
					{
						Cmd.Target->ApplyParameters(*Cmd.Params);
					}
					break;
				case FRelayCommand::EType::Play:
					Cmd.Target->Play();
					break;
				case FRelayCommand::EType::Stop:
					Cmd.Target->Stop();
					break;
				}
			}
		});
	}
} // namespace UE::Subsonic
