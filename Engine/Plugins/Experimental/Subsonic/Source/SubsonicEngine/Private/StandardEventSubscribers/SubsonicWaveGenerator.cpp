// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicWaveGenerator.h"

#include "AudioMixerDevice.h"
#include "DSP/MultichannelBuffer.h"
#include "Sound/SoundGenerator.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveProxyPlayer.h"

namespace UE::Subsonic
{
	namespace WaveGeneratorPrivate
	{
		void Interleave(const Audio::FMultichannelBufferView& Input, float* Output, const int32 NumOutputSamples)
		{
			check(Input.IsEmpty() || NumOutputSamples == Input.Num() * Input[0].Num());
			for (int32 ChannelIndex = 0; ChannelIndex < Input.Num(); ++ChannelIndex)
			{
				float* OutputSample = Output + ChannelIndex;
				TArrayView<float> InputChannel = Input[ChannelIndex];
				for (float Sample : InputChannel)
				{
					*OutputSample = Sample;
					OutputSample += Input.Num();
				}
			}
		}
	}

	FWaveGenerator::FWaveGenerator(TSharedRef<const FSoundWaveData> SoundWaveData, Audio::FMixerDevice* MixerDevice)
	{
		const float SampleRate = MixerDevice->GetSampleRate();
		FSoundWaveProxyPlayer::FSettings Settings(SampleRate, SoundWaveData->GetNumChannels());
		Player = FSoundWaveProxyPlayer::Create(Settings);
		Player->SetSoundWave(SoundWaveData);
	}

	FWaveGenerator::~FWaveGenerator()
	{
	}

	int32 FWaveGenerator::OnGenerateAudio(float* OutAudio, int32 InNumSamples)
	{
		const int32 NumChannels = GetNumChannels();
		check(InNumSamples >= 0 && NumChannels > 0 && InNumSamples % NumChannels == 0);
		const int32 NumFrames = InNumSamples / NumChannels;

		Audio::FMultichannelBuffer TempBuffer;
		for (int32 i = 0; i < NumChannels; ++i)
		{
			Audio::FAlignedFloatBuffer ChannelBuffer;
			ChannelBuffer.SetNumUninitialized(NumFrames);
			TempBuffer.Add(MoveTemp(ChannelBuffer));
		}

		Audio::FMultichannelBufferView TempBufferView = Audio::MakeMultichannelBufferView(TempBuffer);
		Player->RenderMultiChannelAudio(TempBufferView);
		WaveGeneratorPrivate::Interleave(TempBufferView, OutAudio, InNumSamples);

		return InNumSamples;
	}

	int32 FWaveGenerator::GetNumChannels() const
	{
		return Player->GetSourceNumChannels();
	}

	bool FWaveGenerator::IsFinished() const
	{
		return Player->IsFinished();
	}
} // namespace UE::Subsonic