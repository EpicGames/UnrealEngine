// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundGenerator.h"

namespace Audio
{
	class FMixerDevice;
}

class FSoundWaveData;
class FSoundWaveProxyPlayer;

namespace UE::Subsonic
{

	// A pure wave-file player. Volume scaling and relay communication are handled
	// by the outer FSubsonicGenerator wrapper.
	class FWaveGenerator : public ISoundGenerator
	{
	public:
		FWaveGenerator(TSharedRef<const FSoundWaveData> SoundWaveData, Audio::FMixerDevice* MixerDevice);

		virtual ~FWaveGenerator() override;

		virtual int32 OnGenerateAudio(float* OutAudio, int32 InNumSamples) override;
		virtual int32 GetNumChannels() const override;
		virtual bool IsFinished() const override;

	protected:
		TUniquePtr<FSoundWaveProxyPlayer> Player;
	};
}
