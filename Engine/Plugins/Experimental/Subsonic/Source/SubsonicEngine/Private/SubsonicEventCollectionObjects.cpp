// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicEventCollectionObjects.h"

#include "AudioDevice.h"
#include "Engine/World.h"
#include "SubsonicExecutor.h"
#include "SubsonicHandles.h"


namespace UE::Subsonic
{
	const Core::FSubsonicEventCollectionDefinition& USubsonicEventCollection::GetDefinition() const
	{
		return CollectionDefinition;
	}

#if WITH_EDITOR
	FName USubsonicEventCollection::GetDefinitionPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USubsonicEventCollection, CollectionDefinition);
	}
#endif // WITH_EDITOR

	Core::FCollectionHandle USubsonicEventCollection::GetHandle() const
	{
#if WITH_EDITORONLY_DATA
		FName CollectionName;
		if (const UPackage* CollectionPackage = GetPackage())
		{
			const FTopLevelAssetPath AssetPath(CollectionPackage);
			CollectionName = AssetPath.GetPackageName();
		}
#endif // WITH_EDITORONLY_DATA

		return Core::FCollectionHandle
		{
#if WITH_EDITORONLY_DATA
			.CollectionName = CollectionName,
#endif // WITH_EDITORONLY_DATA
			.CollectionId = CollectionDefinition.GetCollectionId(),
		};
	}

#if WITH_EDITOR
	Core::FSubsonicEventCollectionDefinition& USubsonicEventCollection::GetMutableDefinition()
	{
		return CollectionDefinition;
	}

	void USubsonicEventCollection::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
	{
		Super::PostEditChangeChainProperty(PropertyChangedEvent);

#if WITH_EDITORONLY_DATA
		for (auto* Node = PropertyChangedEvent.PropertyChain.GetHead(); Node; Node = Node->GetNextNode())
		{
			const FName PropName = Node->GetValue()->GetFName();
			if (PropName == Core::FSubsonicEventCollectionDefinition::GetParametersPropertyName()
				|| PropName == Core::FSubsonicEvent::GetParametersPropertyName()
				|| PropName == Core::FSubsonicEventActionBase::GetBindingsParameterName())
			{
				const bool bBindingsRemoved = CollectionDefinition.RemoveStaleBindings(GetHandle(), true /* bPromptRemoval */);
				if (bBindingsRemoved)
				{
					CollectionDefinition.BindAllParameters();
				}
				break;
			}
		}
#endif // WITH_EDITORONLY_DATA
	}

	void USubsonicEventCollection::PostEditUndo()
	{
		Super::PostEditUndo();

		CollectionDefinition.RemoveStaleBindings(GetHandle());
		CollectionDefinition.BindAllParameters();
	}
#endif // WITH_EDITOR


	void USubsonicEventCollection::SetDefinition(Core::FSubsonicEventCollectionDefinition&& Definition)
	{
		CollectionDefinition = MoveTemp(Definition);
	}

	void USubsonicEventCollection::PostInitProperties()
	{
		Super::PostInitProperties();
		CollectionDefinition.AssignId();
	}

	void USubsonicEventCollection::PostLoad()
	{
		checkf(CollectionDefinition.IsValid(), TEXT("Definition should have an assigned Id by this point (via PostInitProperties)"));

		Super::PostLoad();

		Audio::FDeviceId DeviceId = INDEX_NONE;
		if (const UWorld* World = GetWorld())
		{
			if (const FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
			{
				DeviceId = AudioDevice->DeviceID;
			}
		}

		CollectionDefinition.Register(GetHandle(), DeviceId);
	}

	void USubsonicEventCollection::BeginDestroy()
	{
		CollectionDefinition.Unregister();
		Super::BeginDestroy();
	}

	void USubsonicEventCollectionExecutor::BeginDestroy()
	{
		Executor.Reset();
		Super::BeginDestroy();
	}

	USubsonicEventCollectionExecutor* USubsonicEventCollectionExecutor::Create(UObject& Parent, FName Name, const USubsonicEventCollection& InCollection, Audio::FDeviceId DeviceId)
	{
		if (USubsonicEventCollectionExecutor* NewExecutor = NewObject<USubsonicEventCollectionExecutor>(&Parent, Name, RF_Transient))
		{
			class FEventCollectionAccessor : public Core::FSubsonicExecutor::ICollectionAccessor
			{
				// Parent USubsonicEventExecutor contains strong ptr, so no need
				// to induce GC overhead by making this a strong pointer.
				TWeakObjectPtr<const USubsonicEventCollection> Collection;

			public:
				FEventCollectionAccessor(const USubsonicEventCollection& InCol)
					: Collection(TWeakObjectPtr<const USubsonicEventCollection>(&InCol))
				{
				}

				virtual ~FEventCollectionAccessor() = default;

				const Core::FSubsonicEventCollectionDefinition* GetDefinition() const override final
				{
					if (Collection.IsValid())
					{
						return &Collection->GetDefinition();
					}

					return nullptr;
				}

				Core::FCollectionHandle GetHandle() const override final
				{
					if (Collection.IsValid())
					{
						return Collection->GetHandle();
					}

					return Core::FCollectionHandle();
				}
			};

			NewExecutor->Collection = &InCollection;
			NewExecutor->Executor = Core::FSubsonicExecutor::Create(DeviceId, MakeUnique<FEventCollectionAccessor>(InCollection));
			return NewExecutor;
		}

		return nullptr;
	}

	void USubsonicEventCollectionExecutor::ExecuteEvent(FGameplayTag EventTag, ESubsonicExecutionResult& OutResult)
	{
		const bool bExecuted = Executor.IsValid() && Executor->ExecuteEvent(EventTag.GetTagName());
		OutResult = bExecuted ? ESubsonicExecutionResult::Succeeded : ESubsonicExecutionResult::Failed;
	}

	const USubsonicEventCollection& USubsonicEventCollectionExecutor::GetCollectionChecked() const
	{
		check(Collection.Get());
		return *Collection.Get();
	}

	const USubsonicEventCollection* USubsonicEventCollectionExecutor::GetCollection() const
	{
		return Collection.Get();
	}

	const Core::FSubsonicExecutor& USubsonicEventCollectionExecutor::GetExecutor() const
	{
		check(IsValid());
		return *Executor.Get();
	}

	bool USubsonicEventCollectionExecutor::IsValid() const
	{
		return Executor.IsValid() && Executor->IsValid() && GetCollection() != nullptr;
	}

	void USubsonicEventCollectionExecutor::Unregister()
	{
		if (Executor.IsValid())
		{
			Executor->Unregister();
			Executor.Reset();
		}

		Collection = nullptr;
	}
} // namespace UE::Subsonic
