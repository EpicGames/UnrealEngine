// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsonicSubsystem.h"

#include "Algo/Transform.h"
#include "AudioDefines.h"
#include "Engine/Engine.h"
#include "ISubsonicEventRegistry.h"
#include "Misc/Optional.h"
#include "SubsonicEventCollectionObjects.h"
#include "SubsonicEventSubscriberInterface.h"
#include "Subsystems/AudioEngineSubsystem.h"
#include "UObject/GCObject.h"


namespace UE::Subsonic
{
	namespace SubsystemPrivate
	{
		bool TryGetDeviceIdFromWorldObject(const UObject* WorldContextObject, Audio::FDeviceId& OutDeviceId)
		{
			OutDeviceId = INDEX_NONE;
			if (WorldContextObject)
			{
				if (GEngine)
				{
					if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
					{
						OutDeviceId = World->GetAudioDevice().GetDeviceID();
						return true;
					}
				}
			}
			else
			{
				if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
				{
					OutDeviceId = DeviceManager->GetActiveAudioDevice().GetDeviceID();
					return true;
				}
			}

			// Either engine isn't loaded, or context object has no parent world
			return false;
		};

		class FSubsonicEventRegistry : public FGCObject, public Core::ISubsonicEventRegistry
		{
		public:
			FSubsonicEventRegistry() = default;

			virtual ~FSubsonicEventRegistry() = default;

		private:
			void IterateSubscribersMatchingDevice(Audio::FDeviceId DeviceId, TFunctionRef<void(TScriptInterface<Core::ISubsonicEventSubscriberInterface>&)> Func)
			{
				for (TScriptInterface<Core::ISubsonicEventSubscriberInterface>& Subscriber : EventSubscribers)
				{
					TOptional<Audio::FDeviceId> SubscriberDeviceId;
					if (UObject* SubscriberObject = Subscriber.GetObject())
					{
						if (const UAudioEngineSubsystem* Subsystem = Cast<UAudioEngineSubsystem>(SubscriberObject))
						{
							if (DeviceId == Subsystem->GetAudioDeviceId())
							{
								Func(Subscriber);
							}
						}
						else
						{
							Audio::FDeviceId WorldDeviceId = INDEX_NONE;
							if (TryGetDeviceIdFromWorldObject(Subscriber.GetObject(), WorldDeviceId))
							{
								if (WorldDeviceId == DeviceId)
								{
									Func(Subscriber);
								}
							}
						}
					}
				}
			}

		public:
			virtual void OnCollectionRegistered(const Core::FCollectionHandle& InHandle, Audio::FDeviceId DeviceId) override
			{
				IterateSubscribersMatchingDevice(DeviceId, [&InHandle](TScriptInterface<Core::ISubsonicEventSubscriberInterface>& Subscriber)
				{
					Subscriber->OnCollectionRegistered(InHandle);
				});
			}

			virtual void OnCollectionUnregistered(const Core::FCollectionHandle& InHandle, Audio::FDeviceId DeviceId) override
			{
				IterateSubscribersMatchingDevice(DeviceId, [&InHandle](TScriptInterface<Core::ISubsonicEventSubscriberInterface>& Subscriber)
				{
					Subscriber->OnCollectionUnregistered(InHandle);
				});
			}

			virtual void OnEventPreExecute(const Core::FSubsonicExecutor& InExecutor, const Core::FEventHandle& InEventHandle) override
			{
				IterateSubscribersMatchingDevice(InExecutor.GetDeviceId(), [&InExecutor, &InEventHandle](TScriptInterface<Core::ISubsonicEventSubscriberInterface>& Subscriber)
				{
					Subscriber->OnEventPreExecute(InExecutor, InEventHandle);
				});
			}

			virtual void OnEventPostExecute(const Core::FSubsonicExecutor& InExecutor, const Core::FEventHandle& InEventHandle) override
			{
				IterateSubscribersMatchingDevice(InExecutor.GetDeviceId(), [&InExecutor, &InEventHandle](TScriptInterface<Core::ISubsonicEventSubscriberInterface>& Subscriber)
				{
					Subscriber->OnEventPostExecute(InExecutor, InEventHandle);
				});
			}

			virtual void OnEventSubscriberRegistered(Core::ISubsonicEventSubscriberInterface& EventSubscriber) override
			{
				TScriptInterface<Core::ISubsonicEventSubscriberInterface> ScriptInterface;
				ScriptInterface.SetObject(Cast<UObject>(&EventSubscriber));
				ScriptInterface.SetInterface(&EventSubscriber);
				EventSubscribers.AddUnique(ScriptInterface);
			}

			virtual void OnEventSubscriberUnregistered(Core::ISubsonicEventSubscriberInterface& EventSubscriber) override
			{
				TScriptInterface<Core::ISubsonicEventSubscriberInterface> ScriptInterface;
				ScriptInterface.SetObject(Cast<UObject>(&EventSubscriber));
				ScriptInterface.SetInterface(&EventSubscriber);
				EventSubscribers.Remove(ScriptInterface);
			}

			virtual void OnExecutorRegistered(const Core::FSubsonicExecutor& InExecutor) override
			{
				IterateSubscribersMatchingDevice(InExecutor.GetDeviceId(), [&](TScriptInterface<Core::ISubsonicEventSubscriberInterface>& Subscriber)
				{
					Subscriber->OnExecutorRegistered(InExecutor);
				});
			}

			virtual void OnExecutorUnregistered(const Core::FSubsonicExecutor& InExecutor) override
			{
				IterateSubscribersMatchingDevice(InExecutor.GetDeviceId(), [&](TScriptInterface<Core::ISubsonicEventSubscriberInterface>& Subscriber)
				{
					Subscriber->OnExecutorUnregistered(InExecutor);
				});
			}

		private:
			virtual void AddReferencedObjects(FReferenceCollector& Collector) override
			{
				TArray<TObjectPtr<UObject>> Objects;
				Algo::Transform(EventSubscribers, Objects, [](const TScriptInterface<Core::ISubsonicEventSubscriberInterface>& Subscriber)
				{
					return TObjectPtr<UObject>(Subscriber.GetObject());
				});
				Collector.AddReferencedObjects(Objects);
			}

			virtual FString GetReferencerName() const override
			{
				return "FSubsonicEventRegistry";
			}

			TArray<TScriptInterface<Core::ISubsonicEventSubscriberInterface>> EventSubscribers;
		};

	} // namespace SubsystemPrivate

	void USubsonicSubsystem::Initialize(FSubsystemCollectionBase& Collection)
	{
		Super::Initialize(Collection);

		Core::ISubsonicEventRegistry::Initialize(TUniquePtr<Core::ISubsonicEventRegistry>(new SubsystemPrivate::FSubsonicEventRegistry()));
	}

	void USubsonicSubsystem::Deinitialize()
	{
		Super::Deinitialize();
		Core::ISubsonicEventRegistry::Deinitialize();
	}

	USubsonicEventCollectionExecutor* USubsonicSubsystem::CreateExecutorBP(UObject* WorldContextObject, FName Name, const USubsonicEventCollection* InCollection)
	{
		if (InCollection && WorldContextObject)
		{
			Audio::FDeviceId DeviceId = INDEX_NONE;
			if (SubsystemPrivate::TryGetDeviceIdFromWorldObject(WorldContextObject, DeviceId))
			{
				return USubsonicEventCollectionExecutor::Create(*WorldContextObject, Name, *InCollection, DeviceId);
			}
		}

		return nullptr;
	}
} // namespace UE::Subsonic
