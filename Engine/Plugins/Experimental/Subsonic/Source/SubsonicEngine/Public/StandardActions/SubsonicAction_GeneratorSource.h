// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/CoreMiscDefines.h"
#include "Sound/SoundWave.h"
#include "SubsonicEventCollection.h"
#include "SubsonicExecutor.h"
#include "SubsonicParameterStore.h"

#include "SubsonicAction_GeneratorSource.generated.h"

namespace UE::Subsonic
{
	USTRUCT(MinimalAPI, DisplayName = "Play Sound (GeneratorSource)")
		struct FSubsonicEventAction_GeneratorSourcePlay : public Core::FSubsonicEventActionBase
	{
		GENERATED_BODY()

		virtual ~FSubsonicEventAction_GeneratorSourcePlay() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const override;

#if WITH_EDITOR
		virtual FText GetDisplayInfo() const override;
#endif // WITH_EDITOR

	public:
		// Addressable name of Generator source instance.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		FName Name;

		// Sound to play.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		TObjectPtr<USoundWave> Sound;

		// Scope of source accessor.  If set to executor, named source is
		// accessible only by pool of sibling event actions for a given
		// executor instance. If set to global, accesses any named source
		// in the global pool.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		ESubsonicExecutionScope Scope = ESubsonicExecutionScope::Executor;

		// Authored parameters applied to the generator at play time (lowest priority layer).
		UPROPERTY(EditAnywhere, Category = "Parameters")
		FSubsonicParameterStore Parameters;
	};

	USTRUCT(MinimalAPI, DisplayName = "Stop Sound (GeneratorSource)")
	struct FSubsonicEventAction_GeneratorSourceStop : public Core::FSubsonicEventActionBase
	{
		GENERATED_BODY()

		virtual ~FSubsonicEventAction_GeneratorSourceStop() = default;

		virtual void Execute(const Core::FSubsonicExecutor& InExecutor, const Core::FActionHandle& InHandle) const override;

#if WITH_EDITOR
		virtual FText GetDisplayInfo() const override;
#endif // WITH_EDITOR

	public:
		// Addressable name of source instance.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		FName Name;

		// Scope of source accessor.  If set to executor, named source is
		// accessible only by pool of sibling event actions for a given
		// executor instance. If set to global, accesses any named source
		// in the global pool.
		UPROPERTY(EditAnywhere, Category = "Subsonic")
		ESubsonicExecutionScope Scope = ESubsonicExecutionScope::Executor;
	};
} // namespace UE::Subsonic