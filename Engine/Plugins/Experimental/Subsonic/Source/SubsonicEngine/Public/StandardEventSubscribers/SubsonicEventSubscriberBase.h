// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SubsonicEventSubscriberInterface.h"
#include "Subsystems/AudioEngineSubsystem.h"

#include "SubsonicEventSubscriberBase.generated.h"

#define UE_API SUBSONICENGINE_API


namespace UE::Subsonic
{
	// Forward Declarations
	class USubsonicSubsystem;

	UCLASS(MinimalAPI, Abstract)
		class USubsonicEventSubscriberBase : public UAudioEngineSubsystem, public Core::ISubsonicEventSubscriberInterface
	{
		GENERATED_BODY()

	public:
		void Initialize(FSubsystemCollectionBase& Collection);
		void Deinitialize();
	};
} // namespace UE::Subsonic
#undef UE_API // SUBSONICENGINE_API