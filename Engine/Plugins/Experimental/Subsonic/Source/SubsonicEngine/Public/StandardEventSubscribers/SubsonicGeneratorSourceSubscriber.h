// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioMixerGeneratorSource.h"
#include "StandardEventSubscribers/SubsonicEventSubscriberBase.h"
#include "StandardEventSubscribers/SubsonicRelay.h"
#include "SubsonicExecutor.h"
#include "SubsonicParameterStore.h"
#include "SubsonicSubscriberDataStore.h"
#include "Tickable.h"

#include "SubsonicGeneratorSourceSubscriber.generated.h"

// Forward Declarations
class USoundWave;

namespace UE::Subsonic
{
	class FSubsonicGenerator;

	// A source that is no longer addressable from actions but still playing.
	// Paired with its generator so ClearStaleSounds can check completion.
	struct FReleasedSource
	{
		TUniquePtr<Audio::IAudioMixerGeneratorSource> Source;
		TSharedPtr<FSubsonicGenerator> Generator;
	};

	struct FInstanceSources
	{
		// Currently addressable generator source via handle
		TUniquePtr<Audio::IAudioMixerGeneratorSource> ActiveSource;

		// Sources no longer addressable from actions but actively playing.
		TArray<FReleasedSource> ReleasedSources;

		// The active generator, shared with the relay command queue so that
		// SetParameters commands can target it directly on the render thread.
		TSharedPtr<FSubsonicGenerator> Generator;

		FSubsonicParameterStore AuthoredParams;   // from action definition (lowest priority)
		FSubsonicParameterStore RuntimeParams;    // from post-playback SetParameters (highest priority)

		// Merges all three layers (authored < trigger < runtime) and enqueues a
		// SetParameters command into the relay's global command queue.
		void ApplyParameters(FSubsonicRelay& Relay);
	};

	/**
	 * Interface to GeneratorSource playback Subsonic implementation.
	 */
	UCLASS()
	class USubsonicGeneratorSourceSubscriber final : public USubsonicEventSubscriberBase, public FTickableGameObject
	{
		GENERATED_BODY()

	public:
		using FExecutorScopeKey = Core::FExecutorScopeKey;
		using FCollectionHandle = Core::FCollectionHandle;

		USubsonicGeneratorSourceSubscriber();

		//~ Begin USubsystem interface
		virtual void Initialize(FSubsystemCollectionBase& Collection) override;
		virtual void Deinitialize() override;
		//~ End USubsystem interface

		virtual void OnCollectionRegistered(const FCollectionHandle& InEvent) override;
		virtual void OnCollectionUnregistered(const FCollectionHandle& InEvent) override;

		virtual void OnExecutorRegistered(const Core::FSubsonicExecutor& InExecutor) override;
		virtual void OnExecutorUnregistered(const Core::FSubsonicExecutor& InExecutor) override;

		void PlaySound(FName Name, USoundWave& Sound, const FSubsonicParameterStore& Params, bool bReleaseExisting = true);
		void PlaySound(const FExecutorScopeKey& InKey, FName Name, USoundWave& Sound, const FSubsonicParameterStore& Params, bool bReleaseExisting = true);

		void StopSound(FName Name);
		void StopSound(const FExecutorScopeKey& InKey, FName Name);

		void SetParameters(FName SourceName, const FSubsonicParameterStore& Params);
		void SetParameters(const FExecutorScopeKey& InKey, FName SourceName, const FSubsonicParameterStore& Params);

		SUBSONICENGINE_API void SetAuthoredParameters(FName SourceName, FSubsonicParameterStore Params);
		SUBSONICENGINE_API void SetAuthoredParameters(const FExecutorScopeKey& InKey, FName SourceName, FSubsonicParameterStore Params);

		// FTickableGameObject interface
		virtual void Tick(float DeltaTime) override;
		virtual bool IsTickableInEditor() const override { return true; }
		virtual TStatId GetStatId() const override;

	private:
		TSharedRef<FSubsonicRelay> Relay;

		Core::TSubscriberDataStore<FInstanceSources> SourcesDataStore;
	};
} // namespace UE::Subsonic
