// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "SubsonicParameterStore.h"
#include "Templates/SharedPointer.h"

namespace Audio { class FMixerDevice; }

namespace UE::Subsonic
{
	class FSubsonicGenerator;

	// Typed command for the per-device command queue on FSubsonicRelay. Commands are expected to be added on the game thread but can
	//  be safely added from other threads. All commands in one game tick are pushed to the audio render thread in a single batch.
	struct FRelayCommand
	{
		enum class EType : uint8 { SetParameters, Play, Stop };

		EType Type = EType::SetParameters;
		TSharedPtr<FSubsonicParameterStore> Params;      // SetParameters only
		TSharedPtr<FSubsonicGenerator> Target;            // all types
	};

	// Per-device object responsible for communicating state changes to the audio render thread.
	class FSubsonicRelay
	{
	public:
		// Append a command to the pending queue (lock-protected).
		void EnqueueCommand(FRelayCommand&& Cmd);

		// Game thread (called from subscriber Tick): flush pending commands into a single
		// AudioRenderThreadCommand lambda on the mixer device.
		void Tick(Audio::FMixerDevice& MixerDevice);

	private:
		FCriticalSection CommandLock;
		TArray<FRelayCommand> PendingCommands;
	};
} // namespace UE::Subsonic
