// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "SubsonicExecutor.h"

namespace UE::Subsonic
{
	// Utility to find an audio device subsystem from an executor scope key. Free function here
	// in SubsonicEngine to keep SubsonicCore free of AudioDevice dependencies.
	template <typename SubsystemType>
	SubsystemType* FindDeviceSubsystem(const Core::FExecutorScopeKey& InKey)
	{
		const FAudioDevice* AudioDevice = nullptr;
		if (const FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			if (InKey.DeviceId == INDEX_NONE)
			{
				AudioDevice = DeviceManager->GetMainAudioDeviceRaw();
			}
			else
			{
				AudioDevice = DeviceManager->GetAudioDeviceRaw(InKey.DeviceId);
			}
		}

		if (AudioDevice)
		{
			return AudioDevice->GetSubsystem<SubsystemType>();
		}

		return nullptr;
	}
} // namespace UE::Subsonic
