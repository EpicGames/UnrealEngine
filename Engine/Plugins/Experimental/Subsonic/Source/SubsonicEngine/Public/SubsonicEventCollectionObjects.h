// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SubsonicEventCollection.h"
#include "SubsonicExecutor.h"
#include "SubsonicHandles.h"
#include "UObject/Object.h"

#include "SubsonicEventCollectionObjects.generated.h"

#define UE_API SUBSONICENGINE_API

namespace UE::Subsonic
{
	UCLASS(MinimalAPI, BlueprintType)
	class USubsonicEventCollection final : public UObject
	{
		GENERATED_BODY()

	public:
		UE_API const Core::FSubsonicEventCollectionDefinition& GetDefinition() const;

#if WITH_EDITOR
		UE_API static FName GetDefinitionPropertyName();
#endif // WITH_EDITOR

		UE_API Core::FCollectionHandle GetHandle() const;

#if WITH_EDITOR
		UE_API UE_INTERNAL Core::FSubsonicEventCollectionDefinition& GetMutableDefinition();
#endif // WITH_EDITOR

		UE_API void SetDefinition(Core::FSubsonicEventCollectionDefinition&& Definition);

	private:
		UPROPERTY(EditAnywhere, Category = "Subsonic", meta = (ShowOnlyInnerProperties))
		Core::FSubsonicEventCollectionDefinition CollectionDefinition;

		virtual void PostInitProperties() override;
		virtual void PostLoad() override;
		virtual void BeginDestroy() override;

#if WITH_EDITOR
		virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
		virtual void PostEditUndo() override;
#endif // WITH_EDITOR
	};

	UENUM(BlueprintType, meta = (DisplayName = "Execution Result"))
	enum class ESubsonicExecutionResult : uint8
	{
		Succeeded,
		Failed
	};

	// Executor handle for a Subsonic Event Collection.
	UCLASS(MinimalAPI, BlueprintType)
	class USubsonicEventCollectionExecutor final : public UObject
	{
		GENERATED_BODY()

	public:
		virtual void BeginDestroy() override;

		// Creates a transient executor that is bound to the given collection.
		UE_API static USubsonicEventCollectionExecutor* Create(UObject& Parent, FName Name, const USubsonicEventCollection& InCollection, Audio::FDeviceId DeviceId);

		// Executes an event with the given gameplay tag if it exists.
		// Sets OutResult to Succeeded if event was found and executed,
		// Failed if not (enum provided as execution expansion support).
		UFUNCTION(BlueprintCallable, Category = "Subsonic|Collection|Events", meta = (ExpandEnumAsExecs = "OutResult"))
		UE_API void ExecuteEvent(FGameplayTag EventTag, ESubsonicExecutionResult& OutResult);

		// Returns collection executor is bound to.
		UE_API const USubsonicEventCollection* GetCollection() const;

		// Returns collection executor is bound to and asserts if no collection is set.
		UE_API const USubsonicEventCollection& GetCollectionChecked() const;

		// Returns underlying executor UObject wraps.
		UE_API const Core::FSubsonicExecutor& GetExecutor() const;

		// Returns whether this executor is valid (has a valid ID and is bound to an event collection)
		UE_API bool IsValid() const;

		// Unregisters the underlying executor implementation, clearing
		// any state pertaining to the given executor managed by subscribers
		// and releasing association with the provided collection when created.
		UE_API void Unregister();

	private:
		UPROPERTY(Transient, BlueprintReadOnly, Category = "Subsonic", meta = (AllowPrivateAccess))
		TObjectPtr<const USubsonicEventCollection> Collection;

		TSharedPtr<Core::FSubsonicExecutor> Executor;
	};
} // namespace UE::Subsonic

#undef UE_API // SUBSONICENGINE_API
