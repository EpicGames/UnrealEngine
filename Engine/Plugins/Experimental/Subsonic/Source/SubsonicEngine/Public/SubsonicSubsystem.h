// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SubsonicEventCollectionObjects.h"
#include "Subsystems/EngineSubsystem.h"

#include "SubsonicSubsystem.generated.h"

#define UE_API SUBSONICENGINE_API


namespace UE::Subsonic
{
	/**
	 * Interface to the Subsonic plugin.
	 */
	UCLASS(MinimalAPI, BlueprintType)
	class USubsonicSubsystem final : public UEngineSubsystem
	{
		GENERATED_BODY()

	public:
		//~ Begin USubsystem interface
		UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
		UE_API virtual void Deinitialize() override;
		//~ End USubsystem interface

		UFUNCTION(BlueprintCallable, Category = "Subsonic", meta = (DisplayName = "Create Executor", WorldContext = "WorldContextObject"))
		UE_API UPARAM(DisplayName = "Executor") USubsonicEventCollectionExecutor* CreateExecutorBP(UObject* WorldContextObject, FName Name, const USubsonicEventCollection* Collection);
	};
} // namespace UE::Subsonic
#undef UE_API
