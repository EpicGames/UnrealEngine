// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SubsonicEngine : ModuleRules
{
	public SubsonicEngine(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioMixerCore",
				"Core",
				"CoreUObject",
				"Engine",
				"GameplayTags",
				"SubsonicCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioExtensions",
				"AudioMixer",
				"MetasoundEngine",
				"MetasoundFrontend",
				"MetasoundGenerator",
				"MetasoundGraphCore",
				"PropertyBindingUtils",
				"SignalProcessing",
			}
		);

		bAllowUETypesInNamespaces = true;
	}
}
