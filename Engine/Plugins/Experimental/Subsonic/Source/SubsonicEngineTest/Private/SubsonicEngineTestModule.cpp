// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


class FSubsonicEngineTestModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FSubsonicEngineTestModule, SubsonicEngineTest)
