// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "GameplayTagContainer.h"
#include "SubsonicEventCollection.h"
#include "SubsonicEventCollectionObjects.h"


namespace UE::Subsonic::Test
{
	namespace TestPrivate
	{
		// Creates a transient USubsonicEventCollection with a single no-op event registered
		// under the provdied EventTag. Returns null if creation or event registration fails.
		USubsonicEventCollection* CreateCollectionWithEvent(const FGameplayTag EventTag)
		{
			USubsonicEventCollection* Collection = NewObject<USubsonicEventCollection>(GetTransientPackage());
			if (!Collection)
			{
				return nullptr;
			}

			if (!Collection->GetMutableDefinition().AddEvent(EventTag))
			{
				return nullptr;
			}

			return Collection;
		}
	} // namespace TestPrivate

	TEST_CLASS_WITH_FLAGS(SubsonicEventCollectionObjectsTest, "Audio.Subsonic.Engine",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter)
	{
		TEST_METHOD(CreateCollectionWithNoOpEvent)
		{
			USubsonicEventCollection* Collection = TestPrivate::CreateCollectionWithEvent(Core::TAG_SubsonicCore_Event_Play);
			ASSERT_THAT(IsNotNull(Collection));

			if (Collection)
			{
				ASSERT_THAT(IsTrue(Collection->GetDefinition().ContainsEvent(Core::TAG_SubsonicCore_Event_Play)));
			}
		}

		TEST_METHOD(CreateExecutorAndFireNoOpEvent)
		{
			USubsonicEventCollection* Collection = TestPrivate::CreateCollectionWithEvent(Core::TAG_SubsonicCore_Event_Play);
			ASSERT_THAT(IsNotNull(Collection));

			if (Collection)
			{
				USubsonicEventCollectionExecutor* Executor = USubsonicEventCollectionExecutor::Create(
					*GetTransientPackage(), FName("TestExecutor"), *Collection, INDEX_NONE);
				ASSERT_THAT(IsNotNull(Executor));
				
				if (Executor)
				{
					ASSERT_THAT(IsTrue(Executor->IsValid()));

					if (Executor->IsValid())
					{
						const bool bExecuted = Executor->GetExecutor().ExecuteEvent(Core::TAG_SubsonicCore_Event_Play.GetTag().GetTagName());
						ASSERT_THAT(IsTrue(bExecuted));
					}		
				}
			}
		}
	};
} // namespace UE::Subsonic::Test
