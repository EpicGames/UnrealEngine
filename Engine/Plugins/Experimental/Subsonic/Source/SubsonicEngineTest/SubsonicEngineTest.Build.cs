// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SubsonicEngineTest : ModuleRules
{
	public SubsonicEngineTest(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"CQTest",
				"Engine",
				"GameplayTags",
				"SubsonicCore",
				"SubsonicEngine"
			}
		);

		bAllowUETypesInNamespaces = true;
	}
}
