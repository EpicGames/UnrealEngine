{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Subsonic",
	"Description": "Subsonic is a high-level audio authoring and playback system. This plugin is experimental and as such there is no guarantee of backward compatibility.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "SubsonicCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SubsonicEngine",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SubsonicEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SubsonicEngineTest",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "AudioWidgets",
			"Enabled": true
		},
		{
			"Name": "GameplayTagsEditor",
			"Enabled": true
		},
		{
			"Name": "PropertyBindingUtils",
			"Enabled": true
		},
		{
			"Name": "Metasound",
			"Enabled": true
		}
	]
}
